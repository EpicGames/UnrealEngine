// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMediaTextureSample.h"

/**
 * Holds size, tiling, and mip-count information for a single tiled media sequence.
 *
 * Note: This type is engine-internal. It lives in MediaAssets/Internal so that
 * media plugins (ImgMedia, MediaPlate, future preview viewports) can share it
 * without forcing it into the public API surface.
 */
struct FMediaSequenceInfo
{
	/** Name of this sequence. */
	FName Name;

	/** Pixel dimensions of this sequence. */
	FIntPoint Dim = FIntPoint::ZeroValue;

	/** Number of mip levels. */
	int32 NumMipLevels = 0;

	/** Tiling description. */
	FMediaTextureTilingDescription TilingDescription;

	/** True if the sequence has more than one tile in either dimension. */
	FORCEINLINE bool IsTiled() const
	{
		return (TilingDescription.TileNum.X > 1) || (TilingDescription.TileNum.Y > 1);
	}

	/** Get the fractional number of tiles, optionally at a specified mip level. */
	FORCEINLINE FVector2f GetPartialTileNum(const int32 InMipLevel = 0) const
	{
		if (IsTiled())
		{
			ensure(InMipLevel >= 0);
			return (FVector2f(Dim) / FVector2f(TilingDescription.TileSize)) / static_cast<float>(1 << InMipLevel);
		}

		return FVector2f::One();
	}
};
