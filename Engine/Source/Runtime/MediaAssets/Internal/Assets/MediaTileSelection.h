// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"

/**
 * Describes which tiles of a single mip level are visible.
 *
 * Note: Engine-internal. See MediaSequenceInfo.h for the visibility rationale.
 */
struct FMediaTileSelection
{
	/**
	 * Create and initialize a new tile selection.
	 *
	 * @param InNumTilesX Horizontal tile count.
	 * @param InNumTilesY Vertical tile count.
	 * @param bInDefaultVisibility Optional visibility for the entire region.
	 */
	MEDIAASSETS_API FMediaTileSelection(int32 InNumTilesX, int32 InNumTilesY, bool bInDefaultVisibility = false);

	/**
	 * Create and initialize a new tile selection, adjusting tile counts for the specified higher mip level.
	 *
	 * @param InMipZeroResolution Pixel resolution of Mip 0.
	 * @param InTileSize Dimensions of tiles in pixels.
	 * @param InTargetMipLevel Higher target mip level for the selection (usually 1 and above).
	 * @param bInDefaultVisibility Optional visibility for the entire region.
	 */
	MEDIAASSETS_API static FMediaTileSelection CreateForTargetMipLevel(const FIntPoint& InMipZeroResolution, const FIntPoint& InTileSize, int32 InTargetMipLevel, bool bInDefaultVisibility = false);

	FMediaTileSelection() = default;
	~FMediaTileSelection() noexcept = default;

	FMediaTileSelection(const FMediaTileSelection&) = default;
	FMediaTileSelection& operator=(const FMediaTileSelection&) = default;

	FMediaTileSelection(FMediaTileSelection&&) = default;
	FMediaTileSelection& operator=(FMediaTileSelection&&) = default;

	/** True if any tile is visible. */
	MEDIAASSETS_API bool IsAnyVisible() const;

	/** True if a specific tile is visible. */
	MEDIAASSETS_API bool IsVisible(int32 InTileCoordX, int32 InTileCoordY) const;

	/** True if the currently visible tiles are also visible in another selection. */
	MEDIAASSETS_API bool Contains(const FMediaTileSelection& InOther) const;

	/** Combine the current selection with another (assumes they are the same size). */
	MEDIAASSETS_API void Include(const FMediaTileSelection& InOther);

	/** Mark a tile as visible. */
	MEDIAASSETS_API void SetVisible(int32 InTileCoordX, int32 InTileCoordY);

	/** Returns the list of visible tile coordinates, in row and column order. */
	MEDIAASSETS_API TArray<FIntPoint> GetVisibleCoordinates() const;

	/**
	 * Returns a calculated list of contiguous visible tile regions. Only provides regions for the
	 * missing tiles if CurrentTileSelection is specified.
	 */
	MEDIAASSETS_API TArray<FIntRect> GetVisibleRegions(const FMediaTileSelection* InCurrentTileSelection = nullptr) const;

	/** Return the rectangular region bounding the visible tiles. */
	MEDIAASSETS_API FIntRect GetVisibleRegion() const;

	/** Returns the number of visible tiles. */
	MEDIAASSETS_API int32 NumVisibleTiles() const;

	/** Returns the overall dimensions in number of tiles. */
	FIntPoint GetDimensions() const { return Dimensions; }

private:
	/** Flat index for (CoordX, CoordY) given InDim.X stride. */
	FORCEINLINE static int32 ToIndex(const int32 InCoordX, const int32 InCoordY, const FIntPoint& InDim)
	{
		return InCoordY * InDim.X + InCoordX;
	}

	/** Visibility bit per tile, row-major. */
	TBitArray<> Tiles;

	/** Tile counts (X, Y). */
	FIntPoint Dimensions;

	/** Cached result of GetVisibleRegion; rebuilt when bCachedVisibleRegionDirty is true. */
	mutable FIntRect CachedVisibleRegion;

	/** True when CachedVisibleRegion is stale and must be recomputed. */
	mutable bool bCachedVisibleRegionDirty = false;
};
