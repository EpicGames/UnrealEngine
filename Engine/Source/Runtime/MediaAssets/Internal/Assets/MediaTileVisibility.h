// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaTileSelection.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

struct FMediaSequenceInfo;

/**
 * Per-frame tile-visibility output produced by a single visibility provider.
 * Multiple requests are merged (bitwise-OR'd) by a per-loader resolver before
 * being handed to the underlying media reader.
 *
 * Note: Engine-internal. See MediaSequenceInfo.h for the visibility rationale.
 */
struct FMediaTileVisibilityRequest
{
	/** Visible tiles per mip level. */
	TMap<int32, FMediaTileSelection> VisibleTiles;

	/** Minimum mip level to upscale into lower-quality mips, or -1 if disabled. */
	int32 MipLevelToUpscale = -1;

	/** Reset to an empty state, preserving allocated capacity. */
	MEDIAASSETS_API void Reset();

	/** Merge another request into this one (OR tiles, min non-negative upscale level). */
	MEDIAASSETS_API void Merge(const FMediaTileVisibilityRequest& InOther);
};

/**
 * Pure interface implemented by anything that wants to influence which tiles
 * a tiled / mipped media sequence loads (mesh-on-plane, sphere, 2D preview
 * viewport, UMG widget, etc.).
 *
 * Providers are owned by their consumer. The visibility resolver holds them
 * by TWeakPtr and prunes when IsAlive() returns false.
 *
 * Most implementations inherit from FMediaTileVisibilityProvider (below) to pick
 * up the thread-safe default implementation of the OnInputsChanged trio.
 */
class IMediaTileVisibilityProvider
{
public:
	virtual ~IMediaTileVisibilityProvider() = default;

	/**
	 * Append this provider's visible tiles for the given sequence.
	 *
	 * Threading contract: invoked by the resolver without any resolver lock held,
	 * potentially from worker threads, potentially concurrently with other providers.
	 * Implementations MUST NOT call back into the owning resolver synchronously
	 * (RegisterProvider / UnregisterProvider / InvalidateSnapshot / GetSnapshot)
	 * and MUST be safe to call concurrently with other invocations on the same instance.
	 *
	 * @param InSequenceInfo  Description of the sequence (size, mips, tiling).
	 * @param OutRequest      Output - tiles are merged into this; do not clear.
	 */
	virtual void GatherVisibleTiles(const FMediaSequenceInfo& InSequenceInfo,
		FMediaTileVisibilityRequest& OutRequest) const = 0;

	/** Returns true if the underlying consumer is still alive (mesh not garbage-collected, etc). */
	virtual bool IsAlive() const = 0;

	/**
	 * OnInputsChanged signals that consumer-pushed inputs to GatherVisibleTiles have changed
	 * (widget resize, mip pin, etc.) so the next snapshot rebuild may yield a different tile set.
	 *
	 * Implementations must be thread-safe against concurrent Add / Remove / Broadcast on the
	 * same provider. Handlers MUST NOT call back into Add/Remove/Broadcast on the same provider.
	 */
	virtual FDelegateHandle AddOnInputsChangedHandler(const FSimpleDelegate& InHandler) = 0;
	virtual void RemoveOnInputsChangedHandler(FDelegateHandle InHandle) = 0;
	virtual void BroadcastOnInputsChanged() = 0;
};


/**
 * Abstract base providing the standard thread-safe OnInputsChanged trio for
 * IMediaTileVisibilityProvider implementations. Inherit from this rather than
 * IMediaTileVisibilityProvider directly unless you need custom threading.
 */
class FMediaTileVisibilityProvider : public IMediaTileVisibilityProvider
{
public:
	//~ Begin IMediaTileVisibilityProvider
	virtual FDelegateHandle AddOnInputsChangedHandler(const FSimpleDelegate& InHandler) override final
	{
		FScopeLock Lock(&OnInputsChangedMutex);
		return OnInputsChanged.Add(InHandler);
	}

	virtual void RemoveOnInputsChangedHandler(FDelegateHandle InHandle) override final
	{
		FScopeLock Lock(&OnInputsChangedMutex);
		OnInputsChanged.Remove(InHandle);
	}

	virtual void BroadcastOnInputsChanged() override final
	{
		// Snapshot the delegate under the mutex, then dispatch the local copy outside.
		// Holding OnInputsChangedMutex across Broadcast would allow handlers to acquire
		// a second lock (e.g. the resolver's Mutex in InvalidateSnapshot) in an order
		// opposite to Add/RemoveOnInputsChangedHandler callers that take the resolver
		// lock first, producing an AB-BA deadlock.
		FSimpleMulticastDelegate Local;
		{
			FScopeLock Lock(&OnInputsChangedMutex);
			Local = OnInputsChanged;
		}
		Local.Broadcast();
	}
	//~ End IMediaTileVisibilityProvider

private:
	FCriticalSection OnInputsChangedMutex;
	FSimpleMulticastDelegate OnInputsChanged;
};
