// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"

namespace UE::MediaAssets
{
	/**
	 * Returns true if InPath denotes a Windows UNC / network path, by literal-
	 * prefix scan after stripping a leading file:// scheme (case-insensitive).
	 * Detected forms:
	 *   - \\server\share        (standard UNC)
	 *   - //server/share        (forward-slash UNC)
	 *   - \\?\UNC\server\...    (Win32 file namespace UNC)
	 *   - \\.\UNC\server\...    (Win32 device namespace UNC)
	 *   - file://<any of the above>
	 *
	 * Limitations: mapped network drives (e.g. Z:\ from 'net use') cannot be
	 * detected from the path string alone. On non-Windows platforms, network
	 * mounts appear as ordinary local paths (/Volumes/<share> on macOS,
	 * arbitrary mount points on Linux) and are also not detected. Callers that
	 * need cross-platform mount-aware detection must supplement this check.
	 */
	MEDIAASSETS_API bool IsUNCPath(FStringView InPath);

	/**
	 * Returns true if media sources may resolve UNC / network paths.
	 * Controlled by the MediaAssets.AllowNetworkPaths console variable; online
	 * game servers can disable it to harden against UNC path injection
	 * through media-source assets.
	 *
	 * The cvar is read-only at runtime: it must be set in DefaultEngine.ini or
	 * on the command line before the MediaAssets module loads. Console writes
	 * after startup are silently ignored.
	 */
	MEDIAASSETS_API bool AreNetworkPathsAllowed();

	/**
	 * Returns true if InPath is acceptable as a media source path: either it
	 * is not a UNC / network path, or network paths are currently allowed by
	 * the MediaAssets.AllowNetworkPaths console variable.
	 */
	MEDIAASSETS_API bool IsSourcePathAllowed(FStringView InPath);
}
