// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaAssets : ModuleRules
	{
		public MediaAssets(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"SignalProcessing",
					"AudioMixer",
					"AudioExtensions",
                    "Core",
					"CoreUObject",
					"Engine",
					"Media",
					"MediaUtils",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"RenderCore",
					"RHI",
					"Renderer",
				});

			if (Target.bBuildEditor)
			{
				PrivateIncludePathModuleNames.Add("TargetPlatform");
			}
		}
	}
}
