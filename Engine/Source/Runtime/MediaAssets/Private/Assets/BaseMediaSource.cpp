// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseMediaSource.h"

#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/MediaFrameWorkObjectVersion.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseMediaSource)

#if WITH_EDITOR
	#include "Interfaces/ITargetPlatform.h"
#endif


/* UObject interface
 *****************************************************************************/

void UBaseMediaSource::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UBaseMediaSource::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	FString Url = GetUrl();

	if (!Url.IsEmpty())
	{
		Context.AddTag(FAssetRegistryTag("Url", Url, FAssetRegistryTag::TT_Alphabetical));
	}
	
	Context.AddTag(FAssetRegistryTag("Validate", Validate() ? TEXT("True") : TEXT("False"),
		FAssetRegistryTag::TT_Alphabetical));
	Super::GetAssetRegistryTags(Context);
}

void UBaseMediaSource::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITORONLY_DATA
	const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
	if (TargetPlatform)
	{
		// Make sure we setup the player name according to the currently selected platform on saves
		const FName* PlatformPlayerName = PlatformPlayerNames.Find(TargetPlatform->IniPlatformName());
		PlayerName = (PlatformPlayerName != nullptr) ? *PlatformPlayerName : NAME_None;
	}
#endif
	Super::PreSave(ObjectSaveContext);
}

void UBaseMediaSource::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	Ar.UsingCustomVersion(FMediaFrameworkObjectVersion::GUID);

	auto MediaCustomVersion = Ar.CustomVer(FMediaFrameworkObjectVersion::GUID);
	auto SeqCustomVersion = Ar.CustomVer(FSequencerObjectVersion::GUID);

	if (Ar.IsLoading() && (SeqCustomVersion < FSequencerObjectVersion::RenameMediaSourcePlatformPlayers))
	{
#if WITH_EDITORONLY_DATA
		if (!Ar.IsFilterEditorOnly())
		{
			TMap<FGuid, FGuid> DummyPlatformPlayers;
			Ar << DummyPlatformPlayers;
		}
#endif

		FString DummyDefaultPlayer;
		Ar << DummyDefaultPlayer;
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (Ar.IsFilterEditorOnly())
		{
			// No editor data is around
			Ar << PlayerName;
		}
		else
		{
			// Full editor data is present
			IMediaModule* MediaModule = static_cast<IMediaModule*>(FModuleManager::Get().GetModule(TEXT("Media")));

			if (Ar.IsLoading() && MediaCustomVersion < FMediaFrameworkObjectVersion::SerializeGUIDsInMediaSourceInsteadOfPlainNames)
			{
				TMap<FString, FName> OldPlatformPlayerNames;

				// Load old plain text map of platforms to players...
				Ar << OldPlatformPlayerNames;

				// Do filter platforms that we cannot translate so they do not pop up visibly
				// (we will loose them on save anyways as we cannot generate a GUID)
				PlatformPlayerNames.Empty();
				BlindPlatformGuidPlayerNames.Empty();
				if (MediaModule)
				{
					for (auto Entry : OldPlatformPlayerNames)
					{
						FGuid PlatformGuid = MediaModule->GetPlatformGuid(FName(Entry.Key));
						if (PlatformGuid.IsValid())
						{
							PlatformPlayerNames.Add(Entry);
						}
					}
				}
			}
			else
			{
				// Load GUID based new format map and translate things back into plain text as appropriate...
				// (or the reverse on saves)
				TMap<FGuid, FGuid> PlatformGuidPlayers;

				if (Ar.IsSaving())
				{
					if (MediaModule)
					{
						for (auto Entry : PlatformPlayerNames)
						{
							FGuid PlatformGuid = MediaModule->GetPlatformGuid(FName(Entry.Key));
							// Check if we got something (just to be sure, we really expect this to work)
							if (PlatformGuid.IsValid())
							{
								IMediaPlayerFactory* Factory = MediaModule->GetPlayerFactory(Entry.Value);
								if (Factory)
								{
									PlatformGuidPlayers.Add(TTuple<FGuid, FGuid>(PlatformGuid, Factory->GetPlayerPluginGUID()));
								}
							}
						}
					}

					// Move over any blind data we encountered when loading earlier - but avoid overriding any data existing in the object otherwise
					for (auto BlindEntry : BlindPlatformGuidPlayerNames)
					{
						PlatformGuidPlayers.FindOrAdd(BlindEntry.Key, BlindEntry.Value);
					}

					Ar << PlatformGuidPlayers;
				}
				else if(Ar.IsLoading())
				{
					Ar << PlatformGuidPlayers;

					PlatformPlayerNames.Empty();
					BlindPlatformGuidPlayerNames.Empty();
					for (auto Entry : PlatformGuidPlayers)
					{
						FName PlatformName = MediaModule ? MediaModule->GetPlatformName(Entry.Key) : FName();
						IMediaPlayerFactory* Factory = MediaModule ? MediaModule->GetPlayerFactory(Entry.Value) : nullptr;
						// Could we resolve the GUID or is this an unknown platform?
						if (PlatformName.IsValid() && Factory)
						{
							PlatformPlayerNames.Add(TTuple<FString, FName>(PlatformName.ToString(), Factory->GetPlayerName()));
						}
						else
						{
							BlindPlatformGuidPlayerNames.Add(Entry);
						}
					}
				}
			}
		}
#else
		Ar << PlayerName;
#endif
	}
}


/* IMediaOptions interface
 *****************************************************************************/

FName UBaseMediaSource::GetDesiredPlayerName() const
{
#if WITH_EDITORONLY_DATA
	const FString RunningPlatformName(FPlatformProperties::IniPlatformName());
	const FName* PlatformPlayerName = PlatformPlayerNames.Find(RunningPlatformName);

	if (PlatformPlayerName == nullptr)
	{
		return Super::GetDesiredPlayerName();
	}

	return *PlatformPlayerName;
#else
	return PlayerName;
#endif
}

