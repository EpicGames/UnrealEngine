// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileMediaSource.h"
#include "MediaAssetsPrivate.h"
#include "MediaSourcePathValidation.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FileMediaSource)

namespace FileMediaSource
{
	/** Name of the PrecacheFile media option. */
	static const FName PrecacheFileOption("PrecacheFile");
}


/* UFileMediaSource interface
 *****************************************************************************/

FString UFileMediaSource::GetFullPath() const
{
	ResolveFullPath();
	return ResolvedFullPath;
}


void UFileMediaSource::SetFilePath(const FString& Path)
{
	ClearResolvedFullPath();
	if (Path.IsEmpty() || Path.StartsWith(TEXT("./")))
	{
		FilePath = Path;
	}
	else
	{
		FString FullPath = FPaths::ConvertRelativePathToFull(Path);
		const FString FullGameContentDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());

		if (FullPath.StartsWith(FullGameContentDir))
		{
			FPaths::MakePathRelativeTo(FullPath, *FullGameContentDir);
			FullPath = FString("./") + FullPath;
		}

		FilePath = FullPath;
	}
}

#if WITH_EDITOR
void UFileMediaSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Update the full path before resuming the PostEditChangeProperty pipeline in case something else needs to consume the full path
	// as part of the property update
	bool bGenerateThumbnail = false;
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UFileMediaSource, FilePath))
	{
		ClearResolvedFullPath();
		bGenerateThumbnail = true;
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (bGenerateThumbnail)
	{
		GenerateThumbnail();
	}
}
#endif

void UFileMediaSource::ClearResolvedFullPath() const
{
	ResolvedFullPath.Empty();
}

void UFileMediaSource::ResolveFullPath() const
{
	if (!ResolvedFullPath.IsEmpty())
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		if (!UE::MediaAssets::IsSourcePathAllowed(ResolvedFullPath))
		{
			UE_LOGF(LogMediaAssets, Warning, "Rejecting non-local file path: %ls", *ResolvedFullPath);
			ResolvedFullPath.Empty();
		}
	};

	ResolvedFullPath = FilePath;// prevent reentry on the fail case

	if (!FPaths::IsRelative(FilePath))
	{
		return;
	}
    
	if (ResolvedFullPath.StartsWith(TEXT("./")))
	{
		// Support Content or Project relative (in that order)
		const TArray<FString> PossibleBasePaths =
		{
			FPaths::ProjectContentDir(),
			FPaths::ProjectDir()
		};

		for (const FString& BasePath : PossibleBasePaths)
		{
			ResolvedFullPath = FPaths::Combine(BasePath, FilePath.RightChop(2));
			FString FinalFullPath = FPaths::ConvertRelativePathToFull(ResolvedFullPath);
			if (FPaths::FileExists(FinalFullPath))
			{
				ResolvedFullPath = FinalFullPath;
				return;
			}
		}
    }

	const TArray<FString>& RootDirectories = FPlatformMisc::GetAdditionalRootDirectories();
    FString RelativeToRootPath = ResolvedFullPath;
    if ( ResolvedFullPath.StartsWith( FPaths::GetRelativePathToRoot() ) )
    {
        // if we start with the relative path to root then remove that so we can change the root directory below
        RelativeToRootPath = ResolvedFullPath.Mid( FPaths::GetRelativePathToRoot().Len());
    }
    for (const FString& RootPath : RootDirectories)
	{
        FString FinalFullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(RootPath, RelativeToRootPath));
		if (FPaths::FileExists(FinalFullPath))
		{
			ResolvedFullPath = FinalFullPath;
		}
	}
    
}

/* IMediaSource overrides
 *****************************************************************************/

bool UFileMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == FileMediaSource::PrecacheFileOption)
	{
		return PrecacheFile;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


bool UFileMediaSource::HasMediaOption(const FName& Key) const
{
	if (Key == FileMediaSource::PrecacheFileOption)
	{
		return true;
	}

	return Super::HasMediaOption(Key);
}

/* UMediaSource overrides
 *****************************************************************************/

FString UFileMediaSource::GetUrl() const
{
	return FString("file://") + GetFullPath();
}

bool UFileMediaSource::Validate() const
{
	ResolveFullPath();

	// ResolvedFullPath is empty when FilePath is empty or when resolution rejected a UNC.
	check( !ResolvedFullPath.IsEmpty()
		|| FilePath.IsEmpty()
		|| !UE::MediaAssets::AreNetworkPathsAllowed() );

	return FPaths::FileExists(ResolvedFullPath);
}

#if WITH_EDITOR
void UFileMediaSource::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	OutInfoElements.Add(FInfoElement(
		NSLOCTEXT("FileMediaSource", "MediaConfigurationHeader", "Media Configuration"),
		NSLOCTEXT("FileMediaSource", "FilePathLabel", "File Path"),
		FText::FromString(GetFullPath())));
}
#endif

