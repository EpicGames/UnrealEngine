// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaComponent.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "IMediaEventSink.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaComponent)

UMediaComponent::UMediaComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bAutoActivate = true;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		MediaPlayer = NewObject<UMediaPlayer>(this, "MediaPlayer", RF_Transient);
		MediaPlayer->SetLooping(false);
		MediaPlayer->PlayOnOpen = false;

		MediaTexture = NewObject<UMediaTexture>(this, "MediaTexture", RF_Transient);
	}
}


UMediaPlayer* UMediaComponent::GetMediaPlayer() const
{
	return MediaPlayer;
}


UMediaTexture* UMediaComponent::GetMediaTexture() const
{
	return MediaTexture;
}

void UMediaComponent::OnRegister()
{
	Super::OnRegister();

	if (MediaPlayer)
	{
		if (MediaTexture)
		{
			MediaTexture->SetMediaPlayer(MediaPlayer);
			MediaTexture->UpdateResource();
		}
	}
}

