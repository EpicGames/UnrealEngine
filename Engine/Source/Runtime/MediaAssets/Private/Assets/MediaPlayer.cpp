// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayer.h"
#include "MediaAssetsPrivate.h"

#include "Engine/Engine.h"
#include "IMediaClock.h"
#include "IMediaControls.h"
#include "IMediaModule.h"
#include "IMediaPlayer.h"
#include "IMediaPlayerFactory.h"
#include "IMediaTicker.h"
#include "IMediaTracks.h"
#include "LatentActions.h"
#include "MediaPlayerFacade.h"
#include "MediaPlayerOptions.h"
#include "MediaHelpers.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "StreamMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlayer)

#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif


FLazyName UMediaPlayer::MediaInfoNameSourceNumMips(TEXT("SourceNumMips"));
FLazyName UMediaPlayer::MediaInfoNameSourceNumTiles(TEXT("SourceNumTiles"));
FLazyName UMediaPlayer::MediaInfoNameStartTimecodeValue(TEXT("StartTimecodeValue"));
FLazyName UMediaPlayer::MediaInfoNameStartTimecodeFrameRate(TEXT("StartTimecodeFrameRate"));
FLazyName UMediaPlayer::MediaInfoNameKeyframeInterval(TEXT("KeyframeInterval"));
FLazyName UMediaPlayer::MediaInfoNameVideoDecoderProvider(TEXT("VideoDecoderProvider"));
FLazyName UMediaPlayer::MediaInfoNameAudioDecoderProvider(TEXT("AudioDecoderProvider"));


/* UMediaPlayer structors
 *****************************************************************************/

UMediaPlayer::UMediaPlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CacheAhead(FTimespan::FromMilliseconds(100))
	, CacheBehind(FTimespan::FromMilliseconds(3000))
	, CacheBehindGame(FTimespan::FromMilliseconds(100))
	, PlayOnOpen(true)
	, Shuffle(false)
	, Loop(false)
	, Playlist(nullptr)
	, PlaylistIndex(INDEX_NONE)
	, TimeDelay(FTimespan::Zero())
	, HorizontalFieldOfView(90.0f)
	, VerticalFieldOfView(60.0f)
	, ViewRotation(FRotator::ZeroRotator)
	, PlayerGuid(IsTemplate() ? FGuid() : FGuid::NewGuid())
	, PlayOnNext(false)
	, RegisteredWithMediaModule(false)
#if WITH_EDITORONLY_DATA
	, AffectedByPIEHandling(true)
	, WasPlayingInPIE(false)
#endif
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PlayerFacade = MakeShareable(new FMediaPlayerFacade(this));
		PlayerFacade->OnMediaEvent().AddUObject(this, &UMediaPlayer::HandlePlayerMediaEvent);
	}
}


/* UMediaPlayer interface
 *****************************************************************************/

bool UMediaPlayer::CanPause() const
{
	return PlayerFacade->CanPause();
}


bool UMediaPlayer::CanPlaySource(UMediaSource* MediaSource)
{
	if ((MediaSource == nullptr) || !MediaSource->Validate())
	{
		return false;
	}

	UE_LOGF(LogMediaAssets, Verbose, "%ls.CanPlaySource %ls", *GetFName().ToString(), *MediaSource->GetFName().ToString());

	return PlayerFacade->CanPlayUrl(MediaSource->GetUrl(), MediaSource);
}


bool UMediaPlayer::CanPlayUrl(const FString& Url)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.CanPlayUrl %ls", *GetFName().ToString(), *Url);

	if (Url.IsEmpty())
	{
		return false;
	}

	return PlayerFacade->CanPlayUrl(Url, GetDefault<UMediaSource>());
}


void UMediaPlayer::EnsurePlaylist() const
{
	if (!Playlist)
	{
		SetPlaylistInternal(NewObject<UMediaPlaylist>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Transient));
	}
}


void UMediaPlayer::SetPlaylistInternal(UMediaPlaylist* InPlaylist) const
{
	if (Playlist && Playlist != InPlaylist && Playlist->IsRooted())
	{
		// To avoid leaking UObjects we need to remove the old playlist from root set
		// (which has been most likely rooted because this MediaPlayer is in disregard for GC set)
		Playlist->RemoveFromRoot();
	}

	Playlist = InPlaylist;

	if (InPlaylist && GUObjectArray.IsDisregardForGC(this))
	{
		// If this MediaPlayer object is in disregard for GC set, add the new Playlist to root set.
		Playlist->AddToRoot();
	}
}

void UMediaPlayer::Close()
{
	UE_LOGF(LogMediaAssets, VeryVerbose, "%ls.Close", *GetFName().ToString());

	PlayerFacade->Close();

	SetPlaylistInternal(nullptr);

	PlaylistIndex = INDEX_NONE;
	PlayOnNext = false;
}

static const FName MediaModuleName("Media");
void UMediaPlayer::CleanUpBeforeDestroy()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);

		if (MediaModule != nullptr)
		{
			UnregisterWithMediaModule();
			MediaModule->GetTicker().RemoveTickable(PlayerFacade.ToSharedRef());
		}

		PlayerFacade->Close();
	}
}


int32 UMediaPlayer::GetAudioTrackChannels(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetAudioTrackChannels(TrackIndex, FormatIndex);
}


int32 UMediaPlayer::GetAudioTrackSampleRate(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetAudioTrackSampleRate(TrackIndex, FormatIndex);
}


FString UMediaPlayer::GetAudioTrackType(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetAudioTrackType(TrackIndex, FormatIndex);
}


FName UMediaPlayer::GetDesiredPlayerName() const
{
	return PlayerFacade->DesiredPlayerName;
}


FTimespan UMediaPlayer::GetDuration() const
{
	return PlayerFacade->GetDuration();
}


float UMediaPlayer::GetHorizontalFieldOfView() const
{
	float OutHorizontal = 0.0f;
	float OutVertical = 0.0f;

	if (!PlayerFacade->GetViewField(OutHorizontal, OutVertical))
	{
		return 0.0f;
	}

	return OutHorizontal;
}


FText UMediaPlayer::GetMediaName() const
{
	return PlayerFacade->GetMediaName();
}


int32 UMediaPlayer::GetNumTracks(EMediaPlayerTrack TrackType) const
{
	return PlayerFacade->GetNumTracks((EMediaTrackType)TrackType);
}


int32 UMediaPlayer::GetNumTrackFormats(EMediaPlayerTrack TrackType, int32 TrackIndex) const
{
	return PlayerFacade->GetNumTrackFormats((EMediaTrackType)TrackType, TrackIndex);
}


FVariant UMediaPlayer::GetMediaInfo(FName InfoName) const
{
	return PlayerFacade->GetMediaInfo(InfoName);
}

TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> UMediaPlayer::GetMediaMetadata() const
{
	return PlayerFacade->GetMediaMetadata();
}

TMap<FString, FMediaMetadataItemsBPT> UMediaPlayer::GetMediaMetadataItems() const
{
	TMap<FString, FMediaMetadataItemsBPT> OutMetadata;
	if(const TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>* MetadataPtr = GetMediaMetadata().Get())
	{
		for(const TPair<FString, TArray<TUniquePtr<IMediaMetadataItem>>>& Data : *MetadataPtr)
		{
			FMediaMetadataItemsBPT OutItems;
			for(const TUniquePtr<IMediaMetadataItem>& Item : Data.Value)
			{
				if(const IMediaMetadataItem* ItemPtr = Item.Get())
				{
					FMediaMetadataItemBPT OutItem;
					OutItem.LanguageCode = ItemPtr->GetLanguageCode();
					OutItem.MimeType = ItemPtr->GetMimeType();
					FVariant TempItem = ItemPtr->GetValue();

					switch (TempItem.GetType())
					{
					case EVariantTypes::String:
						{
							OutItem.StringData = TempItem.GetValue<FString>();
							break;
						}
					case EVariantTypes::Ansichar:
					case EVariantTypes::ByteArray:
						{
							OutItem.BinaryData = TempItem.GetValue<TArray<uint8>>();
							break;
						}

					case EVariantTypes::Bool:
						{
							OutItem.StringData = TempItem.GetValue<bool>() ? TEXT("true") : TEXT("false");
							break;
						}
					case EVariantTypes::Double:
						{
							OutItem.StringData = FString::SanitizeFloat(TempItem.GetValue<double>());
							break;
						}
					case EVariantTypes::Float:
						{
							OutItem.StringData = FString::SanitizeFloat(TempItem.GetValue<float>());
							break;
						}
					case EVariantTypes::UInt8:
						{
							OutItem.StringData = FString::Printf(TEXT("%d"), TempItem.GetValue<uint8>());
							break;
						}
					case EVariantTypes::Int8:
						{
							OutItem.StringData = FString::Printf(TEXT("%d"), TempItem.GetValue<int8>());
							break;
						}
					case EVariantTypes::UInt16:
						{
							OutItem.StringData = FString::Printf(TEXT("%d"), TempItem.GetValue<uint16>());
							break;
						}
					case EVariantTypes::Int16:
						{
							OutItem.StringData = FString::Printf(TEXT("%d"), TempItem.GetValue<int16>());
							break;
						}
					case EVariantTypes::UInt32:
						{
							OutItem.StringData = FString::Printf(TEXT("%d"), TempItem.GetValue<int32>());
							break;
						}
					case EVariantTypes::Int32:
						{
							OutItem.StringData = FString::Printf(TEXT("%d"), TempItem.GetValue<uint32>());
							break;
						}
					case EVariantTypes::UInt64:
						{
							OutItem.StringData = FString::Printf(TEXT("%llu"), TempItem.GetValue<uint64>());
							break;
						}
					case EVariantTypes::Int64:
						{
							OutItem.StringData = FString::Printf(TEXT("%lld"), TempItem.GetValue<int64>());
							break;
						}
					default:
						UE_LOGF(LogMediaAssets, Warning, "GetMediaMetadataItems() unhandled FVariant type!");
					}
					OutItems.Items.Add(OutItem);
				}
			}
			OutMetadata.Add(Data.Key, OutItems);
		}
	}
	return OutMetadata;
}

TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe> UMediaPlayer::GetPlayerFacade() const
{
	return PlayerFacade.ToSharedRef();
}


FName UMediaPlayer::GetPlayerName() const
{
	return PlayerFacade->GetPlayerName();
}


float UMediaPlayer::GetRate() const
{
	return PlayerFacade->GetRate();
}


int32 UMediaPlayer::GetSelectedTrack(EMediaPlayerTrack TrackType) const
{
	return PlayerFacade->GetSelectedTrack((EMediaTrackType)TrackType);
}


void UMediaPlayer::GetSupportedRates(TArray<FFloatRange>& OutRates, bool Unthinned) const
{
	const TRangeSet<float> Rates = PlayerFacade->GetSupportedRates(Unthinned);
	Rates.GetRanges((TArray<TRange<float>>&)OutRates);
}

FTimespan UMediaPlayer::GetTime() const
{
	return PlayerFacade->GetTime();
}

FTimespan UMediaPlayer::GetDisplayTime() const
{
	auto TimeStamp = PlayerFacade->GetDisplayTimeStamp();
	return TimeStamp.IsValid() ? TimeStamp.Time : FTimespan::Zero();
}

UMediaTimeStampInfo* UMediaPlayer::GetTimeStamp() const
{
	UMediaTimeStampInfo *TimeStampInfo = NewObject<UMediaTimeStampInfo>();
	if (TimeStampInfo)
	{
		FMediaTimeStamp TimeStamp = PlayerFacade->GetTimeStamp();
		TimeStampInfo->Time = TimeStamp.Time;
		TimeStampInfo->SequenceIndex = TimeStamp.GetIndexValue();
	}
	return TimeStampInfo;
}

UMediaTimeStampInfo* UMediaPlayer::GetDisplayTimeStamp() const
{
	UMediaTimeStampInfo* TimeStampInfo = NewObject<UMediaTimeStampInfo>();
	if (TimeStampInfo)
	{
		FMediaTimeStamp TimeStamp = PlayerFacade->GetDisplayTimeStamp();
		TimeStampInfo->Time = TimeStamp.Time;
		TimeStampInfo->SequenceIndex = TimeStamp.GetIndexValue();
	}
	return TimeStampInfo;
}

TOptional<FTimecode> UMediaPlayer::GetVideoTimecode() const
{
	return PlayerFacade->GetVideoTimecode();
}


FText UMediaPlayer::GetTrackDisplayName(EMediaPlayerTrack TrackType, int32 TrackIndex) const
{
	return PlayerFacade->GetTrackDisplayName((EMediaTrackType)TrackType, TrackIndex);
}


int32 UMediaPlayer::GetTrackFormat(EMediaPlayerTrack TrackType, int32 TrackIndex) const
{
	return PlayerFacade->GetTrackFormat((EMediaTrackType)TrackType, TrackIndex);
}


FString UMediaPlayer::GetTrackLanguage(EMediaPlayerTrack TrackType, int32 TrackIndex) const
{
	return PlayerFacade->GetTrackLanguage((EMediaTrackType)TrackType, TrackIndex);
}


TOptional<FTimespan> UMediaPlayer::GetTrackDuration(EMediaPlayerTrack TrackType, int32 TrackIndex) const
{
	return PlayerFacade->GetTrackDuration((EMediaTrackType)TrackType, TrackIndex);
}


const FString& UMediaPlayer::GetUrl() const
{
	return PlayerFacade->GetUrl();
}


float UMediaPlayer::GetVerticalFieldOfView() const
{
	float OutHorizontal = 0.0f;
	float OutVertical = 0.0f;

	if (!PlayerFacade->GetViewField(OutHorizontal, OutVertical))
	{
		return 0.0f;
	}

	return OutVertical;
}


float UMediaPlayer::GetVideoTrackAspectRatio(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetVideoTrackAspectRatio(TrackIndex, FormatIndex);
}


FIntPoint UMediaPlayer::GetVideoTrackDimensions(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetVideoTrackDimensions(TrackIndex, FormatIndex);
}


float UMediaPlayer::GetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetVideoTrackFrameRate(TrackIndex, FormatIndex);
}


FFloatRange UMediaPlayer::GetVideoTrackFrameRates(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetVideoTrackFrameRates(TrackIndex, FormatIndex);
}


FString UMediaPlayer::GetVideoTrackType(int32 TrackIndex, int32 FormatIndex) const
{
	return PlayerFacade->GetVideoTrackType(TrackIndex, FormatIndex);
}


FRotator UMediaPlayer::GetViewRotation() const
{
	FQuat OutOrientation;

	if (!PlayerFacade->GetViewOrientation(OutOrientation))
	{
		return FRotator::ZeroRotator;
	}

	return OutOrientation.Rotator();
}


FTimespan UMediaPlayer::GetTimeDelay() const
{
	return PlayerFacade->TimeDelay;
}


bool UMediaPlayer::HasError() const
{
	return PlayerFacade->HasError();
}


bool UMediaPlayer::IsBuffering() const
{
	return PlayerFacade->IsBuffering();
}


bool UMediaPlayer::IsConnecting() const
{
	return PlayerFacade->IsConnecting();
}


bool UMediaPlayer::IsLooping() const
{
	return PlayerFacade->IsLooping();
}


bool UMediaPlayer::IsPaused() const
{
	return PlayerFacade->IsPaused();
}


bool UMediaPlayer::IsPlaying() const
{
	return PlayerFacade->IsPlaying();
}


bool UMediaPlayer::IsPreparing() const
{
	return PlayerFacade->IsPreparing();
}


bool UMediaPlayer::IsClosed() const
{
	return PlayerFacade->IsClosed();
}

bool UMediaPlayer::IsReady() const
{
	return PlayerFacade->IsReady();
}


bool UMediaPlayer::Next()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.Next", *GetFName().ToString());

	if (Playlist == nullptr)
	{
		return false;
	}

	int32 RemainingAttempts = Playlist->Num();

	if (RemainingAttempts == 0)
	{
		return false;
	}

	PlayOnNext |= PlayerFacade->IsPlaying();
	RegisterWithMediaModule();

	while (RemainingAttempts-- > 0)
	{
		UMediaSource* NextSource = Shuffle
			? Playlist->GetRandom(PlaylistIndex)
			: Playlist->GetNext(PlaylistIndex);

		if ((NextSource != nullptr) && NextSource->Validate() && PlayerFacade->Open(NextSource->GetUrl(), NextSource))
		{
			return true;
		}
	}

	return false;
}


bool UMediaPlayer::OpenFile(const FString& FilePath)
{
	Close();

	UE_LOGF(LogMediaAssets, Verbose, "%ls.OpenFile %ls", *GetFName().ToString(), *FilePath);

	EnsurePlaylist();

	if (Playlist == nullptr || !Playlist->AddFile(FilePath))
	{
		return false;
	}

	return Next();
}


bool UMediaPlayer::OpenPlaylistIndex(UMediaPlaylist* InPlaylist, int32 Index)
{
	Close();

	if (InPlaylist == nullptr)
	{
		UE_LOGF(LogMediaAssets, Warning, "%ls.OpenPlaylistIndex called with null MediaPlaylist", *GetFName().ToString());
		return false;
	}

	UE_LOGF(LogMediaAssets, Verbose, "%ls.OpenSource %ls %i", *GetFName().ToString(), *InPlaylist->GetFName().ToString(), Index);

	SetPlaylistInternal(InPlaylist);

	if (Index == INDEX_NONE)
	{
		return true;
	}

	UMediaSource* MediaSource = Playlist->Get(Index);

	if (MediaSource == nullptr)
	{
		UE_LOGF(LogMediaAssets, Warning, "%ls.OpenPlaylistIndex called with invalid PlaylistIndex %i", *GetFName().ToString(), Index);
		return false;
	}

	PlaylistIndex = Index;

	if (!MediaSource->Validate())
	{
		UE_LOGF(LogMediaAssets, Error, "Failed to validate media source %ls (%ls)", *MediaSource->GetName(), *MediaSource->GetUrl());
		return false;
	}

	RegisterWithMediaModule();
	return PlayerFacade->Open(MediaSource->GetUrl(), MediaSource);
}

/**
 * @EventName MediaFramework.MediaSourceOpened
 * @Trigger Triggered when a media source is opened in a media player.
 * @Type Client
 * @Owner MediaIO Team
 */
bool UMediaPlayer::OpenSourceInternal(UMediaSource* MediaSource, const FMediaPlayerOptions* PlayerOptions)
{
	Close();

	if (MediaSource == nullptr)
	{
		UE_LOGF(LogMediaAssets, Warning, "%ls.OpenSource called with null MediaSource", *GetFName().ToString());
		return false;
	}

	UE_LOGF(LogMediaAssets, Verbose, "%ls.OpenSource %ls", *GetFName().ToString(), *MediaSource->GetFName().ToString());

	if (!MediaSource->Validate())
	{
		UE_LOGF(LogMediaAssets, Error, "Failed to validate media source %ls (%ls)", *MediaSource->GetName(), *MediaSource->GetUrl());
		return false;
	}

	EnsurePlaylist();

	if (Playlist == nullptr)
	{
		UE_LOGF(LogMediaAssets, Error, "Failed to create playlist on opening media source %ls (%ls)", *MediaSource->GetName(), *MediaSource->GetUrl());
		return false;
	}

	Playlist->Add(MediaSource);
	PlayOnNext |= PlayerFacade->IsPlaying();
	Playlist->GetNext(PlaylistIndex);

#if WITH_EDITOR
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("MediaSourceType"), MediaSource->GetClass()->GetName()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.MediaSourceOpened"), EventAttributes);
	}
#endif

	RegisterWithMediaModule();
	return PlayerFacade->Open(MediaSource->GetUrl(), MediaSource, PlayerOptions);
}

bool UMediaPlayer::OpenSourceWithOptions(UMediaSource* MediaSource, const FMediaPlayerOptions& PlayerOptions)
{
	return OpenSourceInternal(MediaSource, &PlayerOptions);
}

bool UMediaPlayer::OpenSource(UMediaSource* MediaSource)
{
	return OpenSourceInternal(MediaSource, nullptr);
}

bool UMediaPlayer::OpenUrl(const FString& Url)
{
	Close();

	UE_LOGF(LogMediaAssets, Verbose, "%ls.OpenUrl %ls", *GetFName().ToString(), *Url);

	EnsurePlaylist();

	if (Playlist == nullptr || !Playlist->AddUrl(Url))
	{
		return false;
	}

	return Next();
}


bool UMediaPlayer::Pause()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.Pause", *GetFName().ToString());
	return PlayerFacade->SetRate(0.0f);
}


bool UMediaPlayer::Play()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.Play", *GetFName().ToString());
	return PlayerFacade->SetRate(1.0f);
}

void UMediaPlayer::PlayAndSeek()
{
	PlayOnNext = false;
	if (Play())
	{
		if (PlayerFacade->ActivePlayerOptions.IsSet() && !PlayerFacade->ActivePlayerOptions->SeekTime.IsZero() && SupportsSeeking() && PlayerFacade->ActivePlayerOptions->SeekTimeType == EMediaPlayerOptionSeekTimeType::RelativeToStartTime)
		{
			Seek(PlayerFacade->ActivePlayerOptions->SeekTime);
		}
	}
}


bool UMediaPlayer::Previous()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.Previous", *GetFName().ToString());

	if (Playlist == nullptr)
	{
		return false;
	}

	int32 RemainingAttempts = Playlist->Num();

	if (RemainingAttempts == 0)
	{
		return false;
	}

	PlayOnNext |= PlayerFacade->IsPlaying();
	RegisterWithMediaModule();

	while (--RemainingAttempts >= 0)
	{
		UMediaSource* PrevSource = Shuffle
			? Playlist->GetRandom(PlaylistIndex)
			: Playlist->GetPrevious(PlaylistIndex);

		if ((PrevSource != nullptr) && PrevSource->Validate() && PlayerFacade->Open(PrevSource->GetUrl(), PrevSource))
		{
			return true;
		}
	}

	return false;
}


bool UMediaPlayer::Reopen()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.Reopen", *GetFName().ToString());
	if (Playlist == nullptr)
	{
		return false;
	}
	return OpenPlaylistIndex(Playlist, PlaylistIndex);
}


bool UMediaPlayer::Rewind()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.Rewind", *GetFName().ToString());
	return Seek(FTimespan::Zero());
}


bool UMediaPlayer::Seek(const FTimespan& Time)
{
	UE_LOGF(LogMediaAssets, VeryVerbose, "%ls.Seek %ls", *GetFName().ToString(), *Time.ToString(TEXT("%h:%m:%s.%t")));
	return PlayerFacade->Seek(Time);
}

bool UMediaPlayer::Scrub(const FTimespan& Time)
{
	if (PlayerFacade->IsSeeking())
	{
		UE_LOGF(LogMediaAssets, VeryVerbose, "%ls.Scrub %ls", *GetFName().ToString(), *Time.ToString(TEXT("%h:%m:%s.%t")));
		PlayerFacade->SetNextSeek(Time);
		return true;	// todo: add a failed seek event?
	}
	return Seek(Time);
}

bool UMediaPlayer::SelectTrack(EMediaPlayerTrack TrackType, int32 TrackIndex)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.SelectTrack %ls %i", *GetFName().ToString(), *UEnum::GetValueAsString(TEXT("MediaAssets.EMediaPlayerTrack"), TrackType), TrackIndex);
	return PlayerFacade->SelectTrack((EMediaTrackType)TrackType, TrackIndex);
}


void UMediaPlayer::SetBlockOnTime(const FTimespan& Time)
{
	UE_LOGF(LogMediaAssets, VeryVerbose, "%ls.SetBlockOnTime %ls", *GetFName().ToString(), *Time.ToString(TEXT("%h:%m:%s.%t")));
	return PlayerFacade->SetBlockOnTime(Time);
}


void UMediaPlayer::SetBlockOnTimeRange(const TRange<FTimespan>& TimeRange)
{
	UE_LOGF(LogMediaAssets, VeryVerbose, "%ls.SetBlockOnRange [%ls, %ls]", *GetFName().ToString(),
		*TimeRange.GetLowerBoundValue().ToString(TEXT("%h:%m:%s.%t")), *TimeRange.GetUpperBoundValue().ToString(TEXT("%h:%m:%s.%t")));
	return PlayerFacade->SetBlockOnTimeRange(TimeRange);
}


void UMediaPlayer::SetDesiredPlayerName(FName PlayerName)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetDesiredPlayerName %ls", *GetFName().ToString(), *PlayerName.ToString());
	PlayerFacade->DesiredPlayerName = PlayerName;
}


bool UMediaPlayer::SetLooping(bool Looping)
{
	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetLooping %ls", *GetFName().ToString(), *(Looping ? CoreTexts.True : CoreTexts.False).ToString());

	Loop = Looping;

	return PlayerFacade->SetLooping(Loop && (!Playlist || Playlist->Num() == 1));
}


void UMediaPlayer::SetMediaOptions(const UMediaSource* Options)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetMediaOptions", *GetFName().ToString());
	PlayerFacade->SetMediaOptions(Options);
}


bool UMediaPlayer::SetRate(float Rate)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetRate %f", *GetFName().ToString(), Rate);
	return PlayerFacade->SetRate(Rate);
}


bool UMediaPlayer::SetNativeVolume(float Volume)
{
	return PlayerFacade->SetNativeVolume(Volume);
}


bool UMediaPlayer::SetTrackFormat(EMediaPlayerTrack TrackType, int32 TrackIndex, int32 FormatIndex)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetTrackFormat %ls %i %i", *GetFName().ToString(), *UEnum::GetValueAsString(TEXT("MediaAssets.EMediaPlayerTrack"), TrackType), TrackIndex, FormatIndex);
	return PlayerFacade->SetTrackFormat((EMediaTrackType)TrackType, TrackIndex, FormatIndex);
}


bool UMediaPlayer::SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetVideoTrackFrameRate %i %i %f", *GetFName().ToString(), TrackIndex, FormatIndex, FrameRate);
	return PlayerFacade->SetVideoTrackFrameRate(TrackIndex, FormatIndex, FrameRate);
}

bool UMediaPlayer::SetViewField(float Horizontal, float Vertical, bool Absolute)
{
	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetViewField %f %f %ls", *GetFName().ToString(), Horizontal, Vertical, *(Absolute ? CoreTexts.True : CoreTexts.False).ToString());
	return PlayerFacade->SetViewField(Horizontal, Vertical, Absolute);
}


bool UMediaPlayer::SetViewRotation(const FRotator& Rotation, bool Absolute)
{
	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetViewRotation %ls %ls", *GetFName().ToString(), *Rotation.ToString(), *(Absolute ? CoreTexts.True : CoreTexts.False).ToString());
	return PlayerFacade->SetViewOrientation(FQuat(Rotation), Absolute);
}


void UMediaPlayer::SetTimeDelay(FTimespan InTimeDelay)
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.SetTimeDelay %ls", *GetFName().ToString(), *InTimeDelay.ToString());
	PlayerFacade->TimeDelay = InTimeDelay;
}


bool UMediaPlayer::SupportsRate(float Rate, bool Unthinned) const
{
	return PlayerFacade->SupportsRate(Rate, Unthinned);
}


bool UMediaPlayer::SupportsScrubbing() const
{
	return PlayerFacade->CanScrub();
}


bool UMediaPlayer::SupportsSeeking() const
{
	return PlayerFacade->CanSeek();
}

bool UMediaPlayer::SupportsPlaybackTimeRange() const
{
	return PlayerFacade->SupportsPlaybackTimeRange();
}

TRange<FTimespan> UMediaPlayer::GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet)
{
	return PlayerFacade->GetPlaybackTimeRange(InRangeToGet);
}

bool UMediaPlayer::SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange)
{
	UE_LOGF(LogMediaAssets, VeryVerbose, "%ls.SetPlaybackTimeRange [%ls, %ls]", *GetFName().ToString(),
		*InTimeRange.GetLowerBoundValue().ToString(TEXT("%h:%m:%s.%t")), *InTimeRange.GetUpperBoundValue().ToString(TEXT("%h:%m:%s.%t")));
	return PlayerFacade->SetPlaybackTimeRange(InTimeRange);
}

FFloatInterval UMediaPlayer::GetPlaybackTimeRange(EMediaTimeRangeBPType InRangeToGet)
{
	TRange<FTimespan> r = GetPlaybackTimeRange(static_cast<EMediaTimeRangeType>(InRangeToGet));
	FDoubleInterval i;
	i.Min = !r.IsEmpty() ? r.GetLowerBoundValue().GetTotalSeconds() : -1.0;
	i.Max = !r.IsEmpty() ? r.GetUpperBoundValue().GetTotalSeconds() : -1.0;
	FFloatInterval Imprecise;
	Imprecise.Min = (float) i.Min;
	Imprecise.Max = (float) i.Max;
	return Imprecise;
}

bool UMediaPlayer::SetPlaybackTimeRange(FFloatInterval InTimeRange)
{
	return SetPlaybackTimeRange(TRange<FTimespan>(FTimespan::FromSeconds(InTimeRange.Min), FTimespan::FromSeconds(InTimeRange.Max)));
}


#if WITH_EDITOR

void UMediaPlayer::PausePIE()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.PausePIE", *GetFName().ToString());

	WasPlayingInPIE = IsPlaying();

	if (WasPlayingInPIE)
	{
		Pause();
	}
}


void UMediaPlayer::ResumePIE()
{
	UE_LOGF(LogMediaAssets, Verbose, "%ls.ResumePIE", *GetFName().ToString());

	if (WasPlayingInPIE)
	{
		Play();
	}
}

#endif


/* UObject overrides
 *****************************************************************************/

void UMediaPlayer::BeginDestroy()
{
	CleanUpBeforeDestroy();

	Super::BeginDestroy();
}

FString UMediaPlayer::GetDesc()
{
	return TEXT("UMediaPlayer");
}


void UMediaPlayer::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PlayerGuid = FGuid::NewGuid();
		PlayerFacade->SetGuid(PlayerGuid);

		if (!Playlist)
		{
			Playlist = NewObject<UMediaPlaylist>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Transient);
		}
	}
}


void UMediaPlayer::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Set the player GUID - required for UMediaPlayers dynamically allocated at runtime
		PlayerFacade->SetGuid(PlayerGuid);

		RegisterWithMediaModule();
	}
}


void UMediaPlayer::RegisterWithMediaModule()
{
	if (RegisteredWithMediaModule)
	{
		return;
	}

	IMediaModule* MediaModule = nullptr;
	if (IsInGameThread())
	{
		// LoadModulePtr can't be used on a non-game thread (like the AsyncLoadingThread)
		MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);
	}
	else
	{
		// By the time we get here we should've already called LoadModulePtr above on the game thread (when constructing CDO)
		MediaModule = FModuleManager::GetModulePtr<IMediaModule>(MediaModuleName);
	}

	if (MediaModule != nullptr)
	{
		// Make sure the PlayerFacade instance gets regular tick calls from various spots in the gameloop
		MediaModule->GetClock().AddSink(PlayerFacade.ToSharedRef());
		RegisteredWithMediaModule = true;
	}
	else
	{
		UE_LOGF(LogMediaAssets, Warning, "Failed to register media player '%ls' due to module 'Media' not being loaded yet.", *GetName());
	}
}

void UMediaPlayer::UnregisterWithMediaModule()
{
	if (IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>(MediaModuleName))
	{
		MediaModule->GetClock().RemoveSink(PlayerFacade.ToSharedRef());
		RegisteredWithMediaModule = false;
	}
}

void UMediaPlayer::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PlayerFacade->SetGuid(PlayerGuid);
	}
}


#if WITH_EDITOR

void UMediaPlayer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr)
		? PropertyChangedEvent.Property->GetFName()
		: NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMediaPlayer, Loop))
	{
		SetLooping(Loop);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UMediaPlayer, TimeDelay))
	{
		SetTimeDelay(TimeDelay);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

/* IAudioProxyDataFactory overrides
 *****************************************************************************/

TSharedPtr<Audio::IProxyData> UMediaPlayer::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
//WE GET HERE WHEN STARTING THE GRAPH CREATION ON PLAY (well, at least with the editor) - GAMETHREAD
	if (!Proxy.IsValid())
	{
		Proxy = MakeShared<FMediaPlayerProxy, ESPMode::ThreadSafe>(this);
	}
	return Proxy;
}


/* UMediaPlayer callbacks
 *****************************************************************************/

void UMediaPlayer::HandlePlayerMediaEvent(EMediaEvent Event)
{
	MediaEvent.Broadcast(Event);

	bool bPlayOnOpen = false;

	switch(Event)
	{
	case EMediaEvent::MediaClosed:
		OnMediaClosed.Broadcast();
		break;

	case EMediaEvent::MediaOpened:
		PlayerFacade->SetCacheWindow(CacheAhead, FApp::IsGame() ? CacheBehindGame : CacheBehind);
		if (PlayerFacade->ActivePlayerOptions.IsSet() && PlayerFacade->ActivePlayerOptions->Loop != EMediaPlayerOptionBooleanOverride::UseMediaPlayerSetting)
		{
			PlayerFacade->SetLooping(PlayerFacade->ActivePlayerOptions->Loop == EMediaPlayerOptionBooleanOverride::Enabled);
		}
		else
		{
			PlayerFacade->SetLooping(Loop && (!Playlist || Playlist->Num() == 1));
		}
		PlayerFacade->SetViewField(HorizontalFieldOfView, VerticalFieldOfView, true);
		PlayerFacade->SetViewOrientation(FQuat(ViewRotation), true);
		PlayerFacade->TimeDelay = TimeDelay;

		OnMediaOpened.Broadcast(PlayerFacade->GetUrl());

		if (PlayerFacade->ActivePlayerOptions.IsSet() && PlayerFacade->ActivePlayerOptions->PlayOnOpen != EMediaPlayerOptionBooleanOverride::UseMediaPlayerSetting)
		{
			bPlayOnOpen = (PlayerFacade->ActivePlayerOptions->PlayOnOpen == EMediaPlayerOptionBooleanOverride::Enabled);
		}
		else
		{
			bPlayOnOpen = (PlayOnOpen || PlayOnNext);
		}

		if (bPlayOnOpen)
		{
			PlayAndSeek();
		}
		break;

	case EMediaEvent::MediaOpenFailed:
		OnMediaOpenFailed.Broadcast(PlayerFacade->GetUrl());

		if (Playlist)
		{
			if ((Loop && (Playlist->Num() != 1)) || (PlaylistIndex + 1 < Playlist->Num()))
			{
				Next();
			}
		}
		break;

	case EMediaEvent::PlaybackEndReached:
		OnEndReached.Broadcast();

		if (Playlist)
		{
			if ((Loop && (Playlist->Num() != 1)) || (PlaylistIndex + 1 < Playlist->Num()))
			{
				PlayOnNext = true;
				Next();
			}
		}
		break;

	case EMediaEvent::PlaybackResumed:
		OnPlaybackResumed.Broadcast();
		break;

	case EMediaEvent::PlaybackSuspended:
		OnPlaybackSuspended.Broadcast();
		break;

	case EMediaEvent::SeekCompleted:
		OnSeekCompleted.Broadcast();
		break;

	case EMediaEvent::TracksChanged:
		OnTracksChanged.Broadcast();
		break;

	case EMediaEvent::MetadataChanged:
		OnMetadataChanged.Broadcast();
		break;

	case EMediaEvent::MediaBuffering:
		OnBufferingStart.Broadcast();
		break;

	case EMediaEvent::MediaBufferingComplete:
		OnBufferingCompleted.Broadcast();
		break;
	}
}

class FLatentOpenMediaSourceAction : public FPendingLatentAction
{
public:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;

	TWeakObjectPtr<UMediaPlayer> MediaPlayer;
	float TimeRemaining;
	bool& OutSuccess;
	bool bSawError;
	bool bSawMediaOpened;
	bool bSawMediaOpenFailed;
	bool bSawSeekCompleted;
	FMediaPlayerOptions Options;
	FString URL;

	FLatentOpenMediaSourceAction(const FLatentActionInfo& LatentInfo, UMediaPlayer* InMediaPlayer, UMediaSource* InMediaSource, const FMediaPlayerOptions& InOptions, bool& InSuccess)
		: ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, CallbackTarget(LatentInfo.CallbackTarget)
		, MediaPlayer(InMediaPlayer)
		, TimeRemaining(10.0)
		, OutSuccess(InSuccess)
		, bSawError(false)
		, bSawMediaOpened(false)
		, bSawMediaOpenFailed(false)
		, bSawSeekCompleted(false)
		, Options(InOptions)
	{
		if (InMediaSource)
		{
			URL = InMediaSource->GetUrl();

			InMediaPlayer->OnMediaEvent().AddRaw(this, &FLatentOpenMediaSourceAction::HandleMediaPlayerEvent);
			if (!InMediaPlayer->OpenSourceWithOptions(InMediaSource, InOptions))
			{
				UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Failed initial open: %ls", *URL);
				bSawError = true;
			}
		}
		else
		{
			UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Failed initial open because no media source given");
			bSawError = true;
		}
	}

	virtual ~FLatentOpenMediaSourceAction()
	{
		UnregisterMediaEvent();
	}

	void UnregisterMediaEvent()
	{
		if (MediaPlayer.IsValid())
		{
			MediaPlayer->OnMediaEvent().RemoveAll(this);
		}
	}

	void HandleMediaPlayerEvent(EMediaEvent Event)
	{
		UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: Saw event: %ls", *MediaUtils::EventToString(Event));

		switch (Event)
		{
		case EMediaEvent::MediaOpened:
			bSawMediaOpened = true;
			break;
		case EMediaEvent::MediaOpenFailed:
			bSawMediaOpenFailed = true;
			break;
		case EMediaEvent::SeekCompleted:
			bSawSeekCompleted = true;
			break;
		}
	}

	void FailedOperation(FLatentResponse& Response)
	{
		OutSuccess = false;
		Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		if (bSawMediaOpenFailed)
		{
			UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Saw media open failed event. %ls", *URL);
			FailedOperation(Response);
			return;
		}

		if (!MediaPlayer.IsValid() || MediaPlayer.IsStale())
		{
			UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Media player object was deleted. %ls", *URL);
			FailedOperation(Response);
			return;
		}

		if (bSawError || MediaPlayer->HasError())
		{
			UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Media player is in Error state. %ls", *URL);
			FailedOperation(Response);
			return;
		}

		if (MediaPlayer->IsClosed())
		{
			UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Media player is closed. %ls", *URL);
			FailedOperation(Response);
			return;
		}

		if (MediaPlayer->IsPreparing())
		{
			UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: Is preparing ...");
		}
		else if (MediaPlayer->IsReady())
		{
			UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: IsReady() ... %ls", *URL);

			if (bSawMediaOpened)
			{
				if (!Options.SeekTime.IsZero())
				{
					if (bSawSeekCompleted)
					{
						UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: Triggering output pin after seek completed. Success: %d, %ls", OutSuccess, *URL);
						OutSuccess = true;
						Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
						return;
					}
					else
					{
						if (Options.SeekTime < FTimespan::FromSeconds(0) || Options.SeekTime > MediaPlayer->GetDuration())
						{
							UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Media player seeking to time out of bounds. Seek: %ls, Duration: %ls, URL: %ls",
								*Options.SeekTime.ToString(), *MediaPlayer->GetDuration().ToString(), *URL);
							FailedOperation(Response);
							return;
						}
						UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: Waiting for seek completed event ...");
					}
				}
				else
				{
					UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: Triggering output pin after opened event. Success: %d, %ls", OutSuccess, *URL);
					OutSuccess = true;
					Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
					return;
				}
			}
			else
			{
				UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: Waiting for opened event ...");
			}
		}
		else
		{
			UE_LOGF(LogMediaAssets, Verbose, "Open Media Latent: Waiting for IsReady() ...");
		}

		// Timed out
		TimeRemaining -= Response.ElapsedTime();
		if (TimeRemaining <= 0.0f)
		{
			UE_LOGF(LogMediaAssets, Warning, "Open Media Latent: Timed out. %ls", *URL);
			FailedOperation(Response);
			return;
		}
	}

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override
	{
		return FString::Printf(TEXT("Opening Media: %s"), *URL);
	}
#endif
};

void UMediaPlayer::OpenSourceLatent(const UObject* WorldContextObject, FLatentActionInfo LatentInfo, UMediaSource* MediaSource, const FMediaPlayerOptions& Options, bool& bSuccess)
{
	bSuccess = false;

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FLatentOpenMediaSourceAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			FLatentOpenMediaSourceAction* NewAction = new FLatentOpenMediaSourceAction(LatentInfo, this, MediaSource, Options, bSuccess);
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
		}
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------------------------

FMediaPlayerProxy::FMediaPlayerProxy(UMediaPlayer* Player)
{
	PlayerFacade = Player->GetPlayerFacade();
	TypeHash = GetTypeHash(Player);
}

FMediaPlayerProxy::FMediaPlayerProxy(const FMediaPlayerProxy& Other) = default;

FMediaPlayerProxy::~FMediaPlayerProxy()
{
}
