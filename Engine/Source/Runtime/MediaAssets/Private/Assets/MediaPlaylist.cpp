// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlaylist.h"
#include "MediaAssetsPrivate.h"

#include "Misc/Paths.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "FileMediaSource.h"
#include "StreamMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlaylist)


/* UMediaPlaylist interface
 *****************************************************************************/

bool UMediaPlaylist::Add(UMediaSource* MediaSource)
{
	if (MediaSource == nullptr)
	{
		return false;
	}

	Items.Add(MediaSource);

	return true;
}


bool UMediaPlaylist::AddFile(const FString& FilePath)
{
	if (FilePath.IsEmpty())
	{
		return false;
	}

	const FString FileName = FPaths::GetBaseFilename(FilePath);
	const FName ObjectName = MakeUniqueObjectName(GetTransientPackage(), UFileMediaSource::StaticClass(), FName(*FileName));

	auto MediaSource = NewObject<UFileMediaSource>(GetTransientPackage(), ObjectName, RF_Transactional | RF_Transient);
	MediaSource->SetFilePath(FilePath);

	return Add(MediaSource);
}


bool UMediaPlaylist::AddUrl(const FString& Url)
{
	if (Url.IsEmpty())
	{
		return false;
	}

	auto MediaSource = NewObject<UStreamMediaSource>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Transient);
	MediaSource->StreamUrl = Url;

	if (!MediaSource->Validate())
	{
		UE_LOGF(LogMediaAssets, Error, "Failed to validate URL %ls", *Url);

		return false;
	}

	return Add(MediaSource);
}


UMediaSource* UMediaPlaylist::Get(int32 Index)
{
	if (!Items.IsValidIndex(Index))
	{
		return nullptr;
	}

	return Items[Index];
}


UMediaSource* UMediaPlaylist::GetNext(int32& InOutIndex)
{
	if (Items.Num() > 0)
	{
		InOutIndex = FMath::Clamp(InOutIndex, (int32)INDEX_NONE, Items.Num() - 1) + 1;
		InOutIndex %= Items.Num();

		return Items[InOutIndex];
	}

	InOutIndex = INDEX_NONE;

	return nullptr;
}


UMediaSource* UMediaPlaylist::GetPrevious(int32& InOutIndex)
{
	if (Items.Num() > 0)
	{
		InOutIndex = FMath::Clamp(InOutIndex, 0, Items.Num()) - 1;
		InOutIndex += Items.Num();
		InOutIndex %= Items.Num();

		return Items[InOutIndex];
	}

	InOutIndex = INDEX_NONE;

	return nullptr;
}


UMediaSource* UMediaPlaylist::GetRandom(int32& OutIndex)
{
	if (Items.Num() == 0)
	{
		OutIndex = INDEX_NONE;

		return nullptr;
	}

	OutIndex = FMath::RandHelper(Items.Num() - 1);

	return Items[OutIndex];
}


void UMediaPlaylist::Insert(UMediaSource* MediaSource, int32 Index)
{
	Index = FMath::Clamp(Index, 0, Items.Num());
	Items.Insert(MediaSource, Index);
}


bool UMediaPlaylist::Remove(UMediaSource* MediaSource)
{
	return (Items.Remove(MediaSource) > 0);
}


bool UMediaPlaylist::RemoveAt(int32 Index)
{
	if (!Items.IsValidIndex(Index))
	{
		return false;
	}

	Items.RemoveAt(Index);

	return true;
}


bool UMediaPlaylist::Replace(int32 Index, UMediaSource* Replacement)
{
	if (!Items.IsValidIndex(Index))
	{
		return false;
	}

	Items[Index] = Replacement;

	return true;
}

