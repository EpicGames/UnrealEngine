// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSoundComponent.h"
#include "MediaAssetsPrivate.h"

#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "IMediaAudioSample.h"
#include "IMediaClock.h"
#include "IMediaClockSink.h"
#include "IMediaModule.h"
#include "IMediaPlayer.h"
#include "MediaAudioResampler.h"
#include "Misc/ScopeLock.h"
#include "Sound/AudioSettings.h"
#include "UObject/UObjectGlobals.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaSoundComponent)


DECLARE_FLOAT_COUNTER_STAT(TEXT("MediaUtils MediaSoundComponent Sync"), STAT_MediaUtils_MediaSoundComponentSync, STATGROUP_Media);
DECLARE_FLOAT_COUNTER_STAT(TEXT("MediaUtils MediaSoundComponent SampleTime"), STAT_MediaUtils_MediaSoundComponentSampleTime, STATGROUP_Media);
DECLARE_DWORD_COUNTER_STAT(TEXT("MediaUtils MediaSoundComponent Queued"), STAT_Media_SoundCompQueued, STATGROUP_Media);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(MEDIA_API, MediaStreaming);

/**
 * Clock sink for UMediaSoundComponent.
 */
class FMediaSoundComponentClockSink
	: public IMediaClockSink
{
public:
	FMediaSoundComponentClockSink(UMediaSoundComponent& InOwner)
		: Owner(&InOwner)
	{ }
	virtual ~FMediaSoundComponentClockSink() { }

public:
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override
	{
		if (UMediaSoundComponent* OwnerPtr = Owner.Get())
		{
			OwnerPtr->UpdatePlayer();
		}
	}

private:
	TWeakObjectPtr<UMediaSoundComponent> Owner;
};

FMediaSoundGenerator::FMediaSoundGenerator(FSoundGeneratorParams& InParams)
	: Params(InParams)
{
	// Initialize the settings for the spectrum analyzer
	SpectrumAnalyzer.Init((float)InParams.SampleRate);
	Resampler.Initialize(InParams.NumChannels, InParams.SampleRate);

	Audio::FEnvelopeFollowerInitParams EnvelopeInitParams;
	EnvelopeInitParams.SampleRate = (float)InParams.SampleRate;
	EnvelopeInitParams.NumChannels = 1; //EnvelopeFollower uses mixed down mono buffer
	EnvelopeFollower.Init(EnvelopeInitParams);

	CachedRate = Params.CachedRate;
	CachedTime = Params.CachedTime;
	LastPlaySampleTime = Params.LastPlaySampleTime;

	UE_LOGF(LogMediaAssets, Verbose, "MediaSoundComponent: FMediaSoundGenerator Created.");
}

FMediaSoundGenerator::~FMediaSoundGenerator()
{
	UE_LOGF(LogMediaAssets, Verbose, "MediaSoundComponent: FMediaSoundGenerator Destroyed.");
}

void FMediaSoundGenerator::OnEndGenerate()
{
	UE_LOGF(LogMediaAssets, Verbose, "MediaSoundComponent: OnEndGenerate called.");
	Params.SampleQueue.Reset();
}

void FMediaSoundGenerator::SetCachedData(float InCachedRate, const FTimespan& InCachedTime)
{
	CachedRate = InCachedRate;
	CachedTime = InCachedTime;
}

void FMediaSoundGenerator::SetLastPlaySampleTime(const FTimespan& InLastPlaySampleTime)
{
	LastPlaySampleTime = InLastPlaySampleTime;
}

void FMediaSoundGenerator::SetEnableSpectralAnalysis(bool bInSpectralAnlaysisEnabled)
{
	FScopeLock Lock(&AnalysisCritSect);
	Params.bSpectralAnalysisEnabled = bInSpectralAnlaysisEnabled;
}

void FMediaSoundGenerator::SetEnableEnvelopeFollowing(bool bInEnvelopeFollowingEnabled)
{
	FScopeLock Lock(&AnalysisCritSect);
	Params.bEnvelopeFollowingEnabled = bInEnvelopeFollowingEnabled;
	CurrentEnvelopeValue = 0.0f;
}

void FMediaSoundGenerator::SetSpectrumAnalyzerSettings(Audio::FSpectrumAnalyzerSettings::EFFTSize InFFTSize, const TArray<float>& InFrequenciesToAnalyze)
{
	FScopeLock Lock(&AnalysisCritSect);
	Params.SpectrumAnalyzerSettings.FFTSize = InFFTSize;
	Params.FrequenciesToAnalyze = InFrequenciesToAnalyze;
	SpectrumAnalyzer.SetSettings(Params.SpectrumAnalyzerSettings);
}

void FMediaSoundGenerator::SetEnvelopeFollowingSettings(int32 InAttackTimeMsec, int32 InReleaseTimeMsec)
{
	FScopeLock Lock(&AnalysisCritSect);
	Params.EnvelopeFollowerAttackTime = InAttackTimeMsec;
	Params.EnvelopeFollowerReleaseTime = InReleaseTimeMsec;
	bEnvelopeFollowerSettingsChanged = true;
}

void FMediaSoundGenerator::SetSampleQueue(TSharedPtr<FMediaAudioSampleQueue, ESPMode::ThreadSafe>& InSampleQueue)
{
	FScopeLock Lock(&SampleQueueCritSect);
	Params.SampleQueue = InSampleQueue;
	Params.PreviousSampleQueueFlushCount = Params.SampleQueue.IsValid() ? Params.SampleQueue->GetFlushCount() : 0;

	UE_LOGF(LogMediaAssets, Verbose, "MediaSoundComponent: SetSampleQueue called with new sample queue.");
}

TArray<FMediaSoundComponentSpectralData> FMediaSoundGenerator::GetSpectralData() const
{
	FScopeLock Lock(&AnalysisCritSect);

	if (Params.bSpectralAnalysisEnabled)
	{
		Audio::FAsyncSpectrumAnalyzerScopeLock AnalyzerBufferLock(&SpectrumAnalyzer);

		TArray<FMediaSoundComponentSpectralData> SpectralData;

		for (float Frequency : Params.FrequenciesToAnalyze)
		{
			FMediaSoundComponentSpectralData Data;
			Data.FrequencyHz = Frequency;
			Data.Magnitude = SpectrumAnalyzer.GetMagnitudeForFrequency(Frequency);
			SpectralData.Add(Data);
		}
		return SpectralData;
	}
	return TArray<FMediaSoundComponentSpectralData>();
}

TArray<FMediaSoundComponentSpectralData> FMediaSoundGenerator::GetNormalizedSpectralData() const
{
	FScopeLock Lock(&AnalysisCritSect);

	if (Params.bSpectralAnalysisEnabled)
	{
		Audio::FAsyncSpectrumAnalyzerScopeLock AnalyzerBufferLock(&SpectrumAnalyzer);

		TArray<FMediaSoundComponentSpectralData> SpectralData;

		for (float Frequency : Params.FrequenciesToAnalyze)
		{
			FMediaSoundComponentSpectralData Data;
			Data.FrequencyHz = Frequency;
			Data.Magnitude = SpectrumAnalyzer.GetNormalizedMagnitudeForFrequency(Frequency);
			SpectralData.Add(Data);
		}

		return SpectralData;
	}
	return TArray<FMediaSoundComponentSpectralData>();
}



static const int32 MaxAudioInputSamples = 8;	// accept at most these many samples into our input queue


/* UMediaSoundComponent structors
 *****************************************************************************/

UMediaSoundComponent::UMediaSoundComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Channels(EMediaSoundChannels::Stereo)
	, DynamicRateAdjustment(false)
	, RateAdjustmentFactor(0.00000001f)
	, RateAdjustmentRange(FFloatRange(0.995f, 1.005f))
	, CachedRate(0.0f)
	, CachedTime(FTimespan::Zero())
	, RateAdjustment(1.0f)
	, LastPlaySampleTime(FTimespan::MinValue())
	, EnvelopeFollowerAttackTime(10)
	, EnvelopeFollowerReleaseTime(100)
	, bSpectralAnalysisEnabled(false)
	, bEnvelopeFollowingEnabled(false)
{
	PrimaryComponentTick.bCanEverTick = true;
	bAutoActivate = true;

#if PLATFORM_MAC
	PreferredBufferLength = 4 * 1024; // increase buffer callback size on macOS to prevent underruns
#endif

#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
}


UMediaSoundComponent::~UMediaSoundComponent()
{
	RemoveClockSink();
}


/* UMediaSoundComponent interface
 *****************************************************************************/

bool UMediaSoundComponent::BP_GetAttenuationSettingsToApply(FSoundAttenuationSettings& OutAttenuationSettings)
{
	const FSoundAttenuationSettings* SelectedAttenuationSettings = GetSelectedAttenuationSettings();

	if (SelectedAttenuationSettings == nullptr)
	{
		return false;
	}

	OutAttenuationSettings = *SelectedAttenuationSettings;

	return true;
}


UMediaPlayer* UMediaSoundComponent::GetMediaPlayer() const
{
	return CurrentPlayer.Get();
}


void UMediaSoundComponent::SetMediaPlayer(UMediaPlayer* NewMediaPlayer)
{
	CurrentPlayer = NewMediaPlayer;
}

#if WITH_EDITOR

void UMediaSoundComponent::SetDefaultMediaPlayer(UMediaPlayer* NewMediaPlayer)
{
	MediaPlayer = NewMediaPlayer;
	CurrentPlayer = MediaPlayer;
}

#endif


void UMediaSoundComponent::UpdatePlayer()
{
	UMediaPlayer* CurrentPlayerPtr = CurrentPlayer.Get();
	if (CurrentPlayerPtr == nullptr)
	{
		CachedRate = 0.0f;
		CachedTime = FTimespan::Zero();

		UE::TScopeLock Lock(CriticalSection);
		SampleQueue.Reset();
		MediaSoundGenerator.Reset();
		return;
	}

	// create a new sample queue if the player changed
	TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe> PlayerFacade = CurrentPlayerPtr->GetPlayerFacade();

	// We have some audio decoders which are running with a limited amount of pre-allocated audio sample packets.
	// When the audio packets are not consumed in the FMediaSoundGenerator::OnGenerateAudio method below, these packets are not
	// returned to the decoder which then cannot produce more audio samples.
	//
	// The FMediaSoundGenerator::OnGenerateAudio is only called when our parent USynthComponent it active and
	// this is controlled by USynthComponent::Start() and USynthComponent::Stop(). We are tracking a state change here.
	if (PlayerFacade != CurrentPlayerFacade)
	{
		if (IsActive())
		{
			const auto NewSampleQueue = MakeShared<FMediaAudioSampleQueue, ESPMode::ThreadSafe>(MaxAudioInputSamples);
			PlayerFacade->AddAudioSampleSink(NewSampleQueue);
			{
				UE::TScopeLock Lock(CriticalSection);
				SampleQueue = NewSampleQueue;
				if (MediaSoundGenerator.IsValid())
				{
					static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get())->SetSampleQueue(SampleQueue);
				}
			}
			CurrentPlayerFacade = PlayerFacade;
		}
	}
	else
	{
		// Here, we have a CurrentPlayerFacade set which means are also have a valid FMediaAudioSampleQueue set
		// We need to check for deactivation as it seems there is not callback scheduled when USynthComponent::Stop() is called.
		if(!IsActive())
		{
			UE::TScopeLock Lock(CriticalSection);
			SampleQueue.Reset();
			CurrentPlayerFacade.Reset();
		}
	}

	// caching play rate and time for audio thread (eventual consistency is sufficient)
	CachedRate = PlayerFacade->GetRate();
	CachedTime = PlayerFacade->GetTime();

	if (MediaSoundGenerator.IsValid())
	{
		FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());

		// The play time is derived from the media sound generator's OnGenerate callback
		LastPlaySampleTime = MediaGen->GetLastPlayTime();

		MediaGen->SetCachedData(CachedRate, CachedTime);
		PlayerFacade->SetLastAudioRenderedSampleTime(LastPlaySampleTime);
	}
	else
	{
		PlayerFacade->SetLastAudioRenderedSampleTime(FTimespan::MinValue());
	}
}


/* TAttenuatedComponentVisualizer interface
 *****************************************************************************/

void UMediaSoundComponent::CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const
{
	const FSoundAttenuationSettings* SelectedAttenuationSettings = GetSelectedAttenuationSettings();

	if (SelectedAttenuationSettings != nullptr)
	{
		SelectedAttenuationSettings->CollectAttenuationShapesForVisualization(ShapeDetailsMap);
	}
}


/* UActorComponent interface
 *****************************************************************************/

void UMediaSoundComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	if (SpriteComponent != nullptr)
	{
		SpriteComponent->SpriteInfo.Category = TEXT("Sounds");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Sounds", "Sounds");

		if (bAutoActivate)
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EditorResources/AudioIcons/S_AudioComponent_AutoActivate.S_AudioComponent_AutoActivate")));
		}
		else
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EditorResources/AudioIcons/S_AudioComponent.S_AudioComponent")));
		}
	}
#endif
}

void UMediaSoundComponent::OnUnregister()
{
	{
		UE::TScopeLock Lock(CriticalSection);
		SampleQueue.Reset();
	}
	CurrentPlayerFacade.Reset();
	MediaSoundGenerator.Reset();
	Super::OnUnregister();
}


void UMediaSoundComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	UpdatePlayer();
}


/* USceneComponent interface
 *****************************************************************************/

void UMediaSoundComponent::Activate(bool bReset)
{
	if (bReset || ShouldActivate())
	{
		SetComponentTickEnabled(true);
	}

	Super::Activate(bReset);
}


void UMediaSoundComponent::Deactivate()
{
	if (!ShouldActivate())
	{
		SetComponentTickEnabled(false);
		{
			UE::TScopeLock Lock(CriticalSection);
			SampleQueue.Reset();
		}
		CurrentPlayerFacade.Reset();
		MediaSoundGenerator.Reset();
	}
	Super::Deactivate();
}


/* UObject interface
 *****************************************************************************/


void UMediaSoundComponent::PostLoad()
{
	Super::PostLoad();

	CurrentPlayer = MediaPlayer;
}


#if WITH_EDITOR

void UMediaSoundComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName MediaPlayerName = GET_MEMBER_NAME_CHECKED(UMediaSoundComponent, MediaPlayer);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;

	if (PropertyThatChanged != nullptr)
	{
		const FName PropertyName = PropertyThatChanged->GetFName();

		if (PropertyName == MediaPlayerName)
		{
			CurrentPlayer = MediaPlayer;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif //WITH_EDITOR


/* USynthComponent interface
 *****************************************************************************/

bool UMediaSoundComponent::Init(int32& SampleRate)
{
	Super::Init(SampleRate);

	if (Channels == EMediaSoundChannels::Mono)
	{
		NumChannels = 1;
	}
	else if (Channels == EMediaSoundChannels::Stereo)
	{
		NumChannels = 2;
	}
	else
	{
		NumChannels = 8;
	}

	return true;
}

ISoundGeneratorPtr UMediaSoundComponent::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	FMediaSoundGenerator::FSoundGeneratorParams Params;
	Params.SampleRate = (int32)InParams.SampleRate;
	Params.NumChannels = InParams.NumChannels;

	Params.bSpectralAnalysisEnabled = bSpectralAnalysisEnabled;
	Params.bEnvelopeFollowingEnabled = bEnvelopeFollowingEnabled;
	Params.EnvelopeFollowerAttackTime = EnvelopeFollowerAttackTime;
	Params.EnvelopeFollowerReleaseTime = EnvelopeFollowerReleaseTime;
	Params.SpectrumAnalyzerSettings = SpectrumAnalyzerSettings;
	Params.FrequenciesToAnalyze = FrequenciesToAnalyze;

	Params.CachedRate = CachedRate;
	Params.CachedTime = CachedTime;
	Params.LastPlaySampleTime = LastPlaySampleTime;

	UE::TScopeLock Lock(CriticalSection);
	Params.SampleQueue = SampleQueue;
	Params.PreviousSampleQueueFlushCount = Params.SampleQueue.IsValid() ? Params.SampleQueue->GetFlushCount() : 0;
	return MediaSoundGenerator = ISoundGeneratorPtr(new FMediaSoundGenerator(Params));
}

int32 FMediaSoundGenerator::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	CSV_SCOPED_TIMING_STAT(MediaStreaming, FMediaSoundGenerator_OnGenerateAudio);

	int32 InitialSyncOffset = 0;

	TSharedPtr<FMediaAudioSampleQueue, ESPMode::ThreadSafe> PinnedSampleQueue;
	// Make sure we don't swap the sample queue ptr while we're generating
	{
		FScopeLock Lock(&SampleQueueCritSect);
		PinnedSampleQueue = Params.SampleQueue;
	}

	const float Rate = CachedRate.Load();

	// We have an input queue and are actively playing?
	uint32 CurrentSampleQueueFlushCount = PinnedSampleQueue.IsValid() ? PinnedSampleQueue->GetFlushCount() : 0;
	bool bStillGood = Rate != 0.0f && PinnedSampleQueue.IsValid() && CurrentSampleQueueFlushCount == Params.PreviousSampleQueueFlushCount;
	Params.PreviousSampleQueueFlushCount = CurrentSampleQueueFlushCount;
	if (bStillGood)
	{
		const FTimespan Time = CachedTime.Load();

		{
			const uint32 FramesRequested = uint32(NumSamples / Params.NumChannels);
			uint32 JumpFrame = MAX_uint32;
			FMediaTimeStamp OutTime = FMediaTimeStamp(FTimespan::Zero());
			uint32 FramesWritten = Resampler.Generate(OutAudio, OutTime, FramesRequested, Rate, Time, *PinnedSampleQueue, JumpFrame);

			// Fill in any gap left as we didn't have enough data
			if (FramesWritten < FramesRequested)
			{
				memset(OutAudio + FramesWritten * Params.NumChannels, 0, (NumSamples - FramesWritten * Params.NumChannels) * sizeof(float));
			}

			if (FramesWritten == 0)
			{
				return NumSamples; // no samples available
			}

			// Update audio time
			LastPlaySampleTime = OutTime.Time;
			PinnedSampleQueue->SetAudioTimeIfEqualFlushCount(FMediaTimeStampSample(OutTime, FPlatformTime::Seconds()), CurrentSampleQueueFlushCount);

			SET_FLOAT_STAT(STAT_MediaUtils_MediaSoundComponentSampleTime, OutTime.Time.GetTotalSeconds());
			SET_DWORD_STAT(STAT_Media_SoundCompQueued, PinnedSampleQueue->Num());
		}

		// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -

		if (Params.bSpectralAnalysisEnabled || Params.bEnvelopeFollowingEnabled)
		{
			float* BufferToUseForAnalysis = nullptr;
			int32 NumFrames = NumSamples;

			if (Params.NumChannels == 2)
			{
				NumFrames = NumSamples / 2;

				// Use the scratch buffer to sum the audio to mono
				AudioScratchBuffer.Reset();
				AudioScratchBuffer.AddUninitialized(NumFrames);
				BufferToUseForAnalysis = AudioScratchBuffer.GetData();
				int32 SampleIndex = 0;
				for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex, SampleIndex += Params.NumChannels)
				{
					BufferToUseForAnalysis[FrameIndex] = 0.5f * (OutAudio[SampleIndex] + OutAudio[SampleIndex + 1]);
				}
			}
			else
			{
				BufferToUseForAnalysis = OutAudio;
			}

			if (Params.bSpectralAnalysisEnabled)
			{
				SpectrumAnalyzer.PushAudio(BufferToUseForAnalysis, NumFrames);
				SpectrumAnalyzer.PerformAsyncAnalysisIfPossible(true);
			}

			{
				FScopeLock ScopeLock(&AnalysisCritSect);
				if (Params.bEnvelopeFollowingEnabled)
				{
					if (bEnvelopeFollowerSettingsChanged)
					{
						EnvelopeFollower.SetAttackTime((float)Params.EnvelopeFollowerAttackTime);
						EnvelopeFollower.SetReleaseTime((float)Params.EnvelopeFollowerReleaseTime);

						bEnvelopeFollowerSettingsChanged = false;
					}

					EnvelopeFollower.ProcessAudio(BufferToUseForAnalysis, NumFrames);

					const TArray<float>& EnvelopeValues = EnvelopeFollower.GetEnvelopeValues();
					if (ensure(EnvelopeValues.Num() > 0))
					{
						CurrentEnvelopeValue = FMath::Clamp(EnvelopeValues[0], 0.f, 1.f);
					}
					else
					{
						CurrentEnvelopeValue = 0.f;
					}
				}
			}
		}
	}
	else
	{
		Resampler.Flush();

		LastPlaySampleTime = FTimespan::MinValue();
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
	}
 	return NumSamples;

}

void UMediaSoundComponent::SetEnableSpectralAnalysis(bool bInSpectralAnalysisEnabled)
{
	bSpectralAnalysisEnabled = bInSpectralAnalysisEnabled;
	if (MediaSoundGenerator.IsValid())
	{
		FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());
		MediaGen->SetEnableSpectralAnalysis(bSpectralAnalysisEnabled);
	}
}

void UMediaSoundComponent::SetSpectralAnalysisSettings(TArray<float> InFrequenciesToAnalyze, EMediaSoundComponentFFTSize InFFTSize)
{
	Audio::FSpectrumAnalyzerSettings::EFFTSize SpectrumAnalyzerSize;

	switch (InFFTSize)
	{
		case EMediaSoundComponentFFTSize::Min_64:
			SpectrumAnalyzerSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Min_64;
			break;

		case EMediaSoundComponentFFTSize::Small_256:
			SpectrumAnalyzerSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Small_256;
			break;

		default:
		case EMediaSoundComponentFFTSize::Medium_512:
			SpectrumAnalyzerSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Medium_512;
			break;

		case EMediaSoundComponentFFTSize::Large_1024:
			SpectrumAnalyzerSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Large_1024;
			break;
	}

	SpectrumAnalyzerSettings.FFTSize = SpectrumAnalyzerSize;
	FrequenciesToAnalyze = InFrequenciesToAnalyze;

	if (MediaSoundGenerator.IsValid())
	{
		FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());
		MediaGen->SetSpectrumAnalyzerSettings(SpectrumAnalyzerSize, InFrequenciesToAnalyze);
	}
}

TArray<FMediaSoundComponentSpectralData> UMediaSoundComponent::GetSpectralData()
{
	if (bSpectralAnalysisEnabled)
	{
		if (MediaSoundGenerator.IsValid())
		{
			FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());
			return MediaGen->GetSpectralData();
		}
	}
	// Empty array if spectrum analysis is not implemented
	return TArray<FMediaSoundComponentSpectralData>();
}

TArray<FMediaSoundComponentSpectralData> UMediaSoundComponent::GetNormalizedSpectralData()
{
	if (bSpectralAnalysisEnabled)
	{
		if (MediaSoundGenerator.IsValid())
		{
			FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());
			return MediaGen->GetNormalizedSpectralData();
		}
	}
	// Empty array if spectrum analysis is not implemented
	return TArray<FMediaSoundComponentSpectralData>();
}

void UMediaSoundComponent::SetEnableEnvelopeFollowing(bool bInEnvelopeFollowing)
{
	bEnvelopeFollowingEnabled = bInEnvelopeFollowing;

	if (MediaSoundGenerator.IsValid())
	{
		FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());
		MediaGen->SetEnableEnvelopeFollowing(bInEnvelopeFollowing);
	}
}

void UMediaSoundComponent::SetEnvelopeFollowingsettings(int32 AttackTimeMsec, int32 ReleaseTimeMsec)
{
	EnvelopeFollowerAttackTime = AttackTimeMsec;
	EnvelopeFollowerReleaseTime = ReleaseTimeMsec;

	if (MediaSoundGenerator.IsValid())
	{
		FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());
		MediaGen->SetEnvelopeFollowingSettings(EnvelopeFollowerAttackTime, EnvelopeFollowerReleaseTime);
	}
}

float UMediaSoundComponent::GetEnvelopeValue() const
{
	if (MediaSoundGenerator.IsValid())
	{
		FMediaSoundGenerator* MediaGen = static_cast<FMediaSoundGenerator*>(MediaSoundGenerator.Get());
		return MediaGen->GetCurrentEnvelopeValue();
	}

	return 0.0f;
}

void UMediaSoundComponent::AddClockSink()
{
	if (!ClockSink.IsValid())
	{
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			ClockSink = MakeShared<FMediaSoundComponentClockSink, ESPMode::ThreadSafe>(*this);
			MediaModule->GetClock().AddSink(ClockSink.ToSharedRef());
		}
	}
}

void UMediaSoundComponent::RemoveClockSink()
{
	if (ClockSink.IsValid())
	{
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->GetClock().RemoveSink(ClockSink.ToSharedRef());
		}

		ClockSink.Reset();
	}
}

/* UMediaSoundComponent implementation
 *****************************************************************************/

const FSoundAttenuationSettings* UMediaSoundComponent::GetSelectedAttenuationSettings() const
{
	if (bOverrideAttenuation)
	{
		return &AttenuationOverrides;
	}

	if (AttenuationSettings != nullptr)
	{
		return &AttenuationSettings->Attenuation;
	}

	return nullptr;
}
