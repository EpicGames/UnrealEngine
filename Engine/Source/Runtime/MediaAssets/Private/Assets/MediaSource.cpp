// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSource.h"

#include "IMediaAssetsModule.h"
#include "MediaAssetsPrivate.h"
#include "MediaTexture.h"
#include "Modules/ModuleManager.h"
#include "Misc/Paths.h"
#include "StreamMediaSource.h"

#if WITH_EDITOR
#include "MediaSourceRendererInterface.h"
#include "UObject/Package.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaSource)

#if WITH_EDITOR

void UMediaSource::GenerateThumbnail()
{
	if (MediaSourceRenderer == nullptr)
	{
		IMediaAssetsModule* MediaAssetsModule = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets");
		if (MediaAssetsModule != nullptr)
		{
			MediaSourceRenderer = MediaAssetsModule->CreateMediaSourceRenderer();
		}
	}

	if (MediaSourceRenderer != nullptr)
	{
		IMediaSourceRendererInterface* Interface = Cast<IMediaSourceRendererInterface>(MediaSourceRenderer);
		if (Interface != nullptr)
		{
			ThumbnailImage = Interface->Open(this);
		}
	}
}

void UMediaSource::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	OutInfoElements.Add(FInfoElement(
		NSLOCTEXT("MediaSource", "MediaConfigurationHeader", "Media Configuration"),
		NSLOCTEXT("MediaSource", "UrlLabel", "Url"),
		FText::FromString(GetUrl())));
}
#endif // WITH_EDITOR

void UMediaSource::RegisterSpawnFromFileExtension(const FString& Extension,
	FMediaSourceSpawnDelegate InDelegate)
{
	TMap<FString, FMediaSourceSpawnDelegate>& Delegates =
		GetSpawnFromFileExtensionDelegates();

	Delegates.Emplace(Extension, InDelegate);
}

void UMediaSource::UnregisterSpawnFromFileExtension(const FString& Extension)
{
	TMap<FString, FMediaSourceSpawnDelegate>& Delegates =
		GetSpawnFromFileExtensionDelegates();

	Delegates.Remove(Extension);
}

UMediaSource* UMediaSource::SpawnMediaSourceForString(const FString& MediaPath, UObject* Outer)
{
	TObjectPtr<UMediaSource> MediaSource = nullptr;

	// Is it a URL?
	bool bIsUrl = MediaPath.Contains(TEXT("://"));
	if (bIsUrl)
	{
		TObjectPtr<UStreamMediaSource> StreamMediaSource = NewObject<UStreamMediaSource>(Outer, NAME_None, RF_Transactional);
		StreamMediaSource->StreamUrl = MediaPath;
		MediaSource = StreamMediaSource;
	}
	else
	{
		// Do we know about this file extension?
		FString FileExtension = FPaths::GetExtension(MediaPath);
		TMap<FString, FMediaSourceSpawnDelegate>& Delegates =
			GetSpawnFromFileExtensionDelegates();
		FMediaSourceSpawnDelegate* Delegate = Delegates.Find(FileExtension);
		if ((Delegate != nullptr) && (Delegate->IsBound()))
		{
			MediaSource = Delegate->Execute(MediaPath, Outer);
		}
	}

	// Validate the media source.
	if (MediaSource != nullptr)
	{
		if (!MediaSource->Validate())
		{
			UE_LOGF(LogMediaAssets, Error, "Failed to validate %ls", *MediaPath);
			MediaSource = nullptr;
		}
	}

	return MediaSource;
}

TMap<FString, FMediaSourceSpawnDelegate>& UMediaSource::GetSpawnFromFileExtensionDelegates()
{
	static TMap<FString, FMediaSourceSpawnDelegate> Delegates;
	return Delegates;
}

void UMediaSource::BeginDestroy()
{
#if WITH_EDITOR
	MediaSourceRenderer = nullptr;
#endif // WITH_EDITOR

	Super::BeginDestroy();
}

/* IMediaOptions interface
 *****************************************************************************/

FName UMediaSource::GetDesiredPlayerName() const
{
	return NAME_None;
}


bool UMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	const FVariant* Variant = GetMediaOptionDefault(Key);
	if ((Variant != nullptr) && (Variant->GetType() == EVariantTypes::Bool))
	{
		return Variant->GetValue<bool>();
	}
	else
	{
		return DefaultValue;
	}
}


double UMediaSource::GetMediaOption(const FName& Key, double DefaultValue) const
{
	const FVariant* Variant = GetMediaOptionDefault(Key);
	if ((Variant != nullptr) && (Variant->GetType() == EVariantTypes::Double))
	{
		return Variant->GetValue<double>();
	}
	else
	{
		return DefaultValue;
	}
}


int64 UMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	const FVariant* Variant = GetMediaOptionDefault(Key);
	if ((Variant != nullptr) && (Variant->GetType() == EVariantTypes::Int64))
	{
		return Variant->GetValue<int64>();
	}
	else
	{
		return DefaultValue;
	}
}


FString UMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	const FVariant* Variant = GetMediaOptionDefault(Key);
	if ((Variant != nullptr) && (Variant->GetType() == EVariantTypes::String))
	{
		return Variant->GetValue<FString>();
	}
	else
	{
		return DefaultValue;
	}
}


FText UMediaSource::GetMediaOption(const FName& Key, const FText& DefaultValue) const
{
	return DefaultValue;
}


TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> UMediaSource::GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const
{
	return DefaultValue;
}


bool UMediaSource::HasMediaOption(const FName& Key) const
{
	return MediaOptionsMap.Contains(Key);
}

const UObject* UMediaSource::ToUObject() const
{
	return this;
}


void UMediaSource::SetMediaOptionBool(const FName& Key, bool Value)
{
	FVariant Variant(Value);
	SetMediaOption(Key, Variant);
}


void UMediaSource::SetMediaOptionFloat(const FName& Key, float Value)
{
	SetMediaOptionDouble(Key, (double)Value);
}


void UMediaSource::SetMediaOptionDouble(const FName& Key, double Value)
{
	FVariant Variant(Value);
	SetMediaOption(Key, Variant);
}


void UMediaSource::SetMediaOptionInt64(const FName& Key, int64 Value)
{
	FVariant Variant(Value);
	SetMediaOption(Key, Variant);
}


void UMediaSource::SetMediaOptionString(const FName& Key, const FString& Value)
{
	FVariant Variant(Value);
	SetMediaOption(Key, Variant);
}


const FVariant* UMediaSource::GetMediaOptionDefault(const FName& Key) const
{
	return MediaOptionsMap.Find(Key);
}


void UMediaSource::SetMediaOption(const FName& Key, FVariant& Value)
{
	MediaOptionsMap.Emplace(Key, Value);
}



