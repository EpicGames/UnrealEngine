// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaTexture.h"

#include "ExternalTexture.h"
#include "IMediaClock.h"
#include "IMediaClockSink.h"
#include "IMediaModule.h"
#include "IMediaPlayer.h"
#include "IMediaTextureSample.h"
#include "Materials/MaterialInterface.h"
#include "MediaAssetsPrivate.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "Misc/MediaTextureResource.h"
#include "Modules/ModuleManager.h"
#include "ObjectCacheContext.h"
#include "RectLightTexture.h"
#include "RenderUtils.h"
#include "RenderingThread.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaTexture)

/* Local helpers
 *****************************************************************************/

/**
 * Media clock sink for media textures.
 */
class FMediaTextureClockSink
	: public IMediaClockSink
{
public:

	FMediaTextureClockSink(UMediaTexture& InOwner)
		: Owner(&InOwner)
	{ }

	virtual ~FMediaTextureClockSink() { }

public:

	virtual void TickRender(FTimespan DeltaTime, FTimespan Timecode) override
	{
		FScopeLock Lock(&CriticalSection);

		if (UMediaTexture* OwnerPtr = Owner.Get())
		{
			OwnerPtr->TickResource(Timecode);
		}
	}

	/**
	 * Call this when the owner is destroyed.
	 */
	void OwnerDestroyed()
	{
		FScopeLock Lock(&CriticalSection);
		Owner.Reset();
	}

private:

	TWeakObjectPtr<UMediaTexture> Owner;

	/** Used to prevent owner destruction happening during tick. */
	FCriticalSection CriticalSection;
};


/* UMediaTexture structors
 *****************************************************************************/

UMediaTexture::UMediaTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AddressX(TA_Clamp)
	, AddressY(TA_Clamp)
	, AutoClear(false)
	, ClearColor(FLinearColor::Black)
	, EnableGenMips(false)
#if WITH_EDITORONLY_DATA
	, NumMips_DEPRECATED(1)
#endif
	, NewStyleOutput(false)
	, CurrentAspectRatio(0.0f)
	, CurrentOrientation(MTORI_Original)
	, DefaultGuid(FGuid::NewGuid())
	, Dimensions(FIntPoint::ZeroValue)
	, LastDimensions(FIntPoint::ZeroValue)
	, bIsCleared(false)
	, Size(0)
	, CachedNextSampleTime(FTimespan::MinValue())
	, TextureNumMips(1)
	, MipMapBias(0.0f)
	, ColorspaceOverride(UE::Color::EColorSpace::None)
{
	NeverStream = true;
	SRGB = true;
}


/* UMediaTexture interface
 *****************************************************************************/

float UMediaTexture::GetAspectRatio() const
{
	if (Dimensions.Y == 0)
	{
		return 0.0f;
	}

	return (float)(Dimensions.X) / Dimensions.Y;
}


int32 UMediaTexture::GetHeight() const
{
	return Dimensions.Y;
}


UMediaPlayer* UMediaTexture::GetMediaPlayer() const
{
	return CurrentPlayer.Get();
}


int32 UMediaTexture::GetWidth() const
{
	return Dimensions.X;
}

int32 UMediaTexture::GetTextureNumMips() const
{
	return TextureNumMips;
}

void UMediaTexture::SetMediaPlayer(UMediaPlayer* NewMediaPlayer)
{
	SetMediaPlayer(NewMediaPlayer, /*SampleQueue*/ nullptr);
}

void UMediaTexture::SetMediaPlayer(UMediaPlayer* NewMediaPlayer, TSharedPtr<FMediaTextureSampleQueue> InSampleQueue)
{
	CurrentPlayer = NewMediaPlayer;
	UpdatePlayerAndQueue(MoveTemp(InSampleQueue));
}

void UMediaTexture::CacheNextAvailableSampleTime(FTimespan InNextSampleTime)
{
	CachedNextSampleTime = InNextSampleTime;
}

#if WITH_EDITOR

void UMediaTexture::SetDefaultMediaPlayer(UMediaPlayer* NewMediaPlayer)
{
	MediaPlayer = NewMediaPlayer;
	CurrentPlayer = MediaPlayer;
}

#endif


/* UTexture interface
 *****************************************************************************/

FTextureResource* UMediaTexture::CreateResource()
{
	if (!ClockSink.IsValid())
	{
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			ClockSink = MakeShared<FMediaTextureClockSink, ESPMode::ThreadSafe>(*this);
			MediaModule->GetClock().AddSink(ClockSink.ToSharedRef());
		}
	}

	if (!NewStyleOutput)
	{
		Filter = (TextureNumMips > 1) ? TF_Trilinear : TF_Bilinear;
	}

	const uint8 MaxNumMips = static_cast<uint8>(GMaxTextureMipCount);
	return new FMediaTextureResource(*this, Dimensions, Size, ClearColor, CurrentGuid.IsValid() ? CurrentGuid : DefaultGuid, EnableGenMips, MaxNumMips, ColorspaceOverride);
}


EMaterialValueType UMediaTexture::GetMaterialType() const
{
	if (NewStyleOutput)
	{
		return MCT_Texture2D;
	}
	return EnableGenMips ? MCT_Texture2D : MCT_TextureExternal;
}


float UMediaTexture::GetSurfaceWidth() const
{
	return (float)Dimensions.X;
}


float UMediaTexture::GetSurfaceHeight() const
{
	return (float)Dimensions.Y;
}


FGuid UMediaTexture::GetExternalTextureGuid() const
{
	if (EnableGenMips)
	{
		return FGuid();
	}
	FScopeLock Lock(&CriticalSection);
	return CurrentRenderedGuid;
}

void UMediaTexture::SetRenderedExternalTextureGuid(const FGuid& InNewGuid)
{
	check(IsInRenderingThread());

	FScopeLock Lock(&CriticalSection);
	CurrentRenderedGuid = InNewGuid;
}


uint32 UMediaTexture::CalcTextureMemorySizeEnum(ETextureMipCount Enum) const
{
	return Size;
}

/* UObject interface
 *****************************************************************************/

void UMediaTexture::BeginDestroy()
{
	if (ClockSink.IsValid())
	{
		// Tell sink we are done.
		ClockSink->OwnerDestroyed(); 

		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->GetClock().RemoveSink(ClockSink.ToSharedRef());
		}

		ClockSink.Reset();
	}

	//Unregister the last rendered Guid
	const FGuid LastRendered = GetExternalTextureGuid();
	if (LastRendered.IsValid())
	{
		ENQUEUE_RENDER_COMMAND(MediaTextureUnregisterGuid)(
			[LastRendered](FRHICommandList& RHICmdList)
			{
				FExternalTextureRegistry::Get().UnregisterExternalTexture(LastRendered);
			});
	}

	Super::BeginDestroy();
}


FString UMediaTexture::GetDesc()
{
	return FString::Printf(TEXT("%ix%i [%s]"), Dimensions.X,  Dimensions.Y, GPixelFormats[PF_B8G8R8A8].Name);
}


void UMediaTexture::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddUnknownMemoryBytes(Size);
}


void UMediaTexture::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		NewStyleOutput = true;
	}
#endif
}

void UMediaTexture::PostLoad()
{
	Super::PostLoad();

	CurrentPlayer = MediaPlayer;
}

bool UMediaTexture::IsPostLoadThreadSafe() const
{
	return false;
}

#if WITH_EDITOR

void UMediaTexture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName AddressXName = GET_MEMBER_NAME_CHECKED(UMediaTexture, AddressX);
	static const FName AddressYName = GET_MEMBER_NAME_CHECKED(UMediaTexture, AddressY);
	static const FName AutoClearName = GET_MEMBER_NAME_CHECKED(UMediaTexture, AutoClear);
	static const FName ClearColorName = GET_MEMBER_NAME_CHECKED(UMediaTexture, ClearColor);
	static const FName MediaPlayerName = GET_MEMBER_NAME_CHECKED(UMediaTexture, MediaPlayer);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	
	if (PropertyThatChanged == nullptr)
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		return;
	}

	const FName PropertyName = PropertyThatChanged->GetFName();

	if (PropertyName == MediaPlayerName)
	{
		CurrentPlayer = MediaPlayer;
	}

	// don't update resource for these properties
	if ((PropertyName == AutoClearName) ||
		(PropertyName == ClearColorName) ||
		(PropertyName == MediaPlayerName))
	{
		UObject::PostEditChangeProperty(PropertyChangedEvent);

		return;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	// notify materials for these properties
	if ((PropertyName == AddressXName) ||
		(PropertyName == AddressYName))
	{
		NotifyMaterials();
	}
}

#endif // WITH_EDITOR


/* UMediaTexture implementation
 *****************************************************************************/

void UMediaTexture::TickResource(FTimespan Timecode)
{
	using namespace UE::MediaTexture::Private;

	if (GetResource() == nullptr)
	{
		return;
	}

	const FGuid PreviousGuid = CurrentGuid;

	// media player bookkeeping
	UpdatePlayerAndQueue();
	
#if WITH_EDITOR
	// Detect texture dimension changes
	if (LastDimensions != Dimensions)
	{
		FObjectCacheContextScope ObjectCache;
		for (UMaterialInterface* MaterialInterface : ObjectCache.GetContext().GetMaterialsAffectedByTexture(this))
		{
			MaterialInterface->RecacheUniformExpressions(false);
		}
		LastDimensions = Dimensions;
	}
#endif

	if (!CurrentPlayer.IsValid())
	{
		if ((LastClearColor == ClearColor) && (LastSrgb == SRGB) && (bIsCleared))
		{
			return; // nothing to render
		}
	}

	LastClearColor = ClearColor;
	LastSrgb = SRGB;

	// set up render parameters
	FMediaTextureResource::FRenderParams RenderParams;

	bool bIsSampleValid = false;
	if (UMediaPlayer* CurrentPlayerPtr = CurrentPlayer.Get())
	{
		if (CurrentPlayerPtr->GetPlayerFacade()->GetPlayer().IsValid())
		{
			if (CurrentPlayerPtr->GetPlayerFacade()->GetPlayer()->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
			{
				/*
					We are using the old-style "sample queue to sink" architecture to actually just pass along only ONE sample at a time from the logic
					inside the player facade to the sinks. The selection as to what to render this frame is expected to be done earlier
					this frame on the gamethread, hence only a single output frame is selected and passed along...
				*/
				TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;
				while (SampleQueue->Dequeue(Sample))
					;

				if (!Sample.IsValid())
				{
					// Player is active (do not clear), but we have no new data
					// -> we do not need to trigger anything on the renderthread
					return;
				}

				UpdateSampleInfo(Sample);

				RenderParams.TextureSample = Sample;

				RenderParams.Rate = CurrentPlayerPtr->GetRate();
				RenderParams.Time = Sample->GetTime();

				// Track the sample's sRGB status and make sure the (high level) texture flags reflect it
				// (this is really only good for Rec.703 type setups and will need enhancements!)
				SRGB = Sample->IsOutputSrgb();
				LastSrgb = SRGB;

				TextureNumMips = static_cast<int32>(CalcTextureNumMips(*Sample, EnableGenMips));
				bIsSampleValid = true;
			}
			else
			{
				//
				// Old style: pass queue along and dequeue only at render time
				//
				const bool PlayerActive = CurrentPlayerPtr->IsPaused() || CurrentPlayerPtr->IsPlaying() || CurrentPlayerPtr->IsPreparing();
				if (PlayerActive)
				{
					TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;
					if (SampleQueue->Peek(Sample))
					{
						UpdateSampleInfo(Sample);

						// See above: track sRGB state (for V1 we don't look at all samples - but this should be fine as this should not change on a per sample basis usually)
						SRGB = Sample->IsOutputSrgb();
						LastSrgb = SRGB;

						TextureNumMips = static_cast<int32>(CalcTextureNumMips(*Sample, EnableGenMips));
						bIsSampleValid = true;
					}

					RenderParams.SampleSource = SampleQueue;

					RenderParams.Rate = CurrentPlayerPtr->GetRate();
					RenderParams.Time = FMediaTimeStamp(CurrentPlayerPtr->GetTime());
				}
				else
				{
					CurrentAspectRatio = 0.0f;
					CurrentOrientation = MTORI_Original;

					if (!AutoClear)
					{
						return; // retain last frame
					}
				}
			}
		}
		else 
		{
			CurrentAspectRatio = 0.0f;
			CurrentOrientation = MTORI_Original;

			if (!AutoClear)
			{
				return; // retain last frame
			}
		}
	}
	else if (!AutoClear && (CurrentGuid == PreviousGuid))
	{
		return; // retain last frame
	}

	// update filter state, responding to mips setting
	if (!NewStyleOutput)
	{
		Filter = (TextureNumMips > 1) ? TF_Trilinear : TF_Bilinear;
	}

	// setup render parameters
	RenderParams.CanClear = AutoClear;
	RenderParams.ClearColor = ClearColor;
	RenderParams.PreviousGuid = PreviousGuid;
	RenderParams.CurrentGuid = CurrentGuid;

	// redraw texture resource on render thread
	FMediaTextureResource* ResourceParam = (FMediaTextureResource*)GetResource();

	const ERenderMode RenderModeParam = GetRenderMode();
	const UTexture* TexturePtrNotDeferenced = this;

	ENQUEUE_RENDER_COMMAND(MediaTextureResourceRender)(
		[ResourceParam, RenderParams, RenderModeParam, TexturePtrNotDeferenced](FRHICommandListImmediate& RHICmdList)
		{
			check(ResourceParam);

			// Lock/Enqueue rect atlas refresh if that texture is used by a rect. light
			RectLightAtlas::FAtlasTextureInvalidationScope InvalidationScope(TexturePtrNotDeferenced);

			if (RenderModeParam == ERenderMode::JustInTime)
			{
				// Cache the render params if this is a just in time render mode. 
				// User must call JustInTimeRender to update the resource.
				ResourceParam->SetJustInTimeRenderParams(RenderParams);
			}
			else
			{
				// Otherwise, render the texture right away
				ResourceParam->ResetJustInTimeRenderParams();
				ResourceParam->Render(RenderParams);
			}
		});

	
	// The texture is cleared if we have auto clear enabled, and we do not have a valid sample.
	bIsCleared = ((AutoClear) && (bIsSampleValid == false));
}

void UMediaTexture::JustInTimeRender()
{
	if (FMediaTextureResource* MediaResource = static_cast<FMediaTextureResource*>(GetResource()))
	{
		MediaResource->JustInTimeRender();
	}
}

void UMediaTexture::UpdateSampleInfo(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> & Sample)
{
	CurrentAspectRatio = (float)Sample->GetAspectRatio();
	switch (Sample->GetOrientation())
	{
		case EMediaOrientation::Original: CurrentOrientation = MTORI_Original; break;
		case EMediaOrientation::CW90: CurrentOrientation = MTORI_CW90; break;
		case EMediaOrientation::CW180: CurrentOrientation = MTORI_CW180; break;
		case EMediaOrientation::CW270: CurrentOrientation = MTORI_CW270; break;
		default: CurrentOrientation = MTORI_Original;
	}
}

void UMediaTexture::UpdatePlayerAndQueue(TSharedPtr<FMediaTextureSampleQueue> InSampleQueue)
{
	if (UMediaPlayer* CurrentPlayerPtr = CurrentPlayer.Get())
	{
		const FGuid PlayerGuid = CurrentPlayerPtr->GetGuid();

		// Player changed?
		if (CurrentGuid != PlayerGuid)
		{
			if (FMediaTextureResource* MediaResource = static_cast<FMediaTextureResource*>(GetResource()))
			{
				MediaResource->FlushPendingData();
			}

			SampleQueue = InSampleQueue ? MoveTemp(InSampleQueue) : MakeShared<FMediaTextureSampleQueue, ESPMode::ThreadSafe>();
			CurrentPlayerPtr->GetPlayerFacade()->AddVideoSampleSink(SampleQueue.ToSharedRef());
			CurrentGuid = PlayerGuid;
		}
	}
	else
	{
		// No player. Did we already reset to default?
		if (CurrentGuid != DefaultGuid)
		{
			// No, do so now...
			SampleQueue.Reset();
			CurrentGuid = DefaultGuid;

			if (FMediaTextureResource* MediaResource = static_cast<FMediaTextureResource*>(GetResource()))
			{
				MediaResource->FlushPendingData();
			}

		}
	}
}

FTimespan UMediaTexture::GetNextSampleTime() const
{
	return CachedNextSampleTime;
}

int32 UMediaTexture::GetAvailableSampleCount() const
{
	return SampleQueue->Num();
}

float UMediaTexture::GetCurrentAspectRatio() const
{
	return CurrentAspectRatio;
}

MediaTextureOrientation UMediaTexture::GetCurrentOrientation() const
{
	return CurrentOrientation;
}

float UMediaTexture::GetMipMapBias() const
{
	// Clamped to the legal DirectX range.
	return FMath::Clamp(MipMapBias, -16.0f, 15.99f);
}

void UMediaTexture::SetMipMapBias(float InMipMapBias)
{
	MipMapBias = InMipMapBias;
}
