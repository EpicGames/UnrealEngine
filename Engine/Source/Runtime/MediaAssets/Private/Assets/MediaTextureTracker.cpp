// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaTextureTracker.h"
#include "MediaTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaTextureTracker)

FMediaTextureTracker& FMediaTextureTracker::Get()
{
	static FMediaTextureTracker Engine;
	return Engine;
}

void FMediaTextureTracker::RegisterTexture(TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>& InInfo,
	TObjectPtr<UMediaTexture> InTexture)
{
	check(IsInGameThread());

	// Do we have this media texture yet?
	TWeakObjectPtr<UMediaTexture> TexturePtr(InTexture);
	if (MediaTextures.Contains(TexturePtr) == false)
	{
		MediaTextures.Emplace(TexturePtr);
		MapTextureToObject.Emplace(TexturePtr);
	}

	// Skip if already registered: Unregister's RemoveSwap removes only one match,
	// so a duplicate registration would leak an entry on the matching unregister.
	TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>* FoundObjects = MapTextureToObject.Find(TexturePtr);
	const bool bAlreadyRegistered = FoundObjects->ContainsByPredicate(
		[&InInfo](const TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>& InObject)
		{
			return InObject.Pin() == InInfo;
		});
	if (bAlreadyRegistered)
	{
		return;
	}

	FoundObjects->Add(InInfo);

	ObjectRegisteredDelegate.Broadcast(InTexture, InInfo);
}

void FMediaTextureTracker::UnregisterTexture(TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>& InInfo,
	TObjectPtr<UMediaTexture> InTexture)
{
	check(IsInGameThread());

	if (InTexture != nullptr)
	{
		TWeakObjectPtr<UMediaTexture> TexturePtr(InTexture);
		if (TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>* FoundObjects = MapTextureToObject.Find(TexturePtr))
		{
			if (FoundObjects->RemoveSwap(InInfo) > 0)
			{
				ObjectUnregisteredDelegate.Broadcast(InTexture, InInfo);

				// Drop the map and MediaTextures entries once the texture has no remaining
				// consumers, so neither structure grows unboundedly across long sessions.
				if (FoundObjects->IsEmpty())
				{
					MapTextureToObject.Remove(TexturePtr);
					MediaTextures.RemoveSwap(TexturePtr);
				}
			}
		}
		return;
	}

	// Fallback for callers that have lost the texture pointer (e.g. the texture was
	// garbage-collected before the owner was destroyed). Scan the map and remove InInfo
	// wherever it appears - typically only one entry, since callers register against a
	// single texture per InInfo. Defer the empty-entry pruning until after the iteration
	// to avoid mutating the map mid-walk.
	TArray<TWeakObjectPtr<UMediaTexture>, TInlineAllocator<1>> EmptiedTextures;
	for (TPair<TWeakObjectPtr<UMediaTexture>, TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>>& Pair : MapTextureToObject)
	{
		if (Pair.Value.RemoveSwap(InInfo) > 0)
		{
			ObjectUnregisteredDelegate.Broadcast(Pair.Key.Get(), InInfo);
			if (Pair.Value.IsEmpty())
			{
				EmptiedTextures.Add(Pair.Key);
			}
		}
	}
	for (const TWeakObjectPtr<UMediaTexture>& Key : EmptiedTextures)
	{
		MapTextureToObject.Remove(Key);
		MediaTextures.RemoveSwap(Key);
	}
}

const TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>* FMediaTextureTracker::GetObjects(TObjectPtr<UMediaTexture> InTexture) const
{
	check(IsInGameThread());

	const TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>* ObjectsPtr = MapTextureToObject.Find(InTexture);
	return ObjectsPtr;
}

