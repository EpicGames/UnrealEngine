// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/MediaTileSelection.h"

#include "Math/UnrealMathUtility.h"

FMediaTileSelection::FMediaTileSelection(int32 InNumTilesX, int32 InNumTilesY, bool bInDefaultVisibility)
	: Tiles(bInDefaultVisibility, InNumTilesX * InNumTilesY)
	, Dimensions(InNumTilesX, InNumTilesY)
	, CachedVisibleRegion()
	, bCachedVisibleRegionDirty(true)
{
}

FMediaTileSelection FMediaTileSelection::CreateForTargetMipLevel(const FIntPoint& InMipZeroResolution, const FIntPoint& InTileSize, int32 InTargetMipLevel, bool bInDefaultVisibility)
{
	ensure(InTargetMipLevel >= 0);

	if (InTileSize.X == 0 || InTileSize.Y == 0)
	{
		return FMediaTileSelection(1, 1, bInDefaultVisibility);
	}

	// Clamp the shift to avoid UB on extreme mip levels - real mip pyramids never approach 31,
	// but a defensive clamp keeps the result deterministic if a caller ever passes a stale value.
	const int32 ClampedMipLevel = FMath::Clamp(InTargetMipLevel, 0, 30);
	const int32 MipLevelDiv = 1 << ClampedMipLevel;
	int32 NumTilesX = FMath::Max(1, FMath::CeilToInt(static_cast<float>(InMipZeroResolution.X) / MipLevelDiv / InTileSize.X));
	int32 NumTilesY = FMath::Max(1, FMath::CeilToInt(static_cast<float>(InMipZeroResolution.Y) / MipLevelDiv / InTileSize.Y));

	return FMediaTileSelection(NumTilesX, NumTilesY, bInDefaultVisibility);
}

bool FMediaTileSelection::IsAnyVisible() const
{
	return Tiles.Contains(true);
}

bool FMediaTileSelection::IsVisible(int32 InTileCoordX, int32 InTileCoordY) const
{
	return Tiles[ToIndex(InTileCoordX, InTileCoordY, Dimensions)];
}

bool FMediaTileSelection::Contains(const FMediaTileSelection& InOther) const
{
	// Cross-dimension queries cannot produce a meaningful result: the bit positions encode
	// different tile addresses on each side. Mirror Include's invariant and bail safely.
	if (!ensureMsgf(Dimensions == InOther.Dimensions,
		TEXT("FMediaTileSelection::Contains dimension mismatch (%dx%d vs %dx%d)"),
		Dimensions.X, Dimensions.Y, InOther.Dimensions.X, InOther.Dimensions.Y))
	{
		return false;
	}

	//Modified version of TBitArray's CompareSetBits() method.

	TBitArray<>::FConstWordIterator ThisIterator(Tiles);
	TBitArray<>::FConstWordIterator OtherIterator(InOther.Tiles);

	ThisIterator.FillMissingBits(0u);
	OtherIterator.FillMissingBits(0u);

	while (ThisIterator || OtherIterator)
	{
		const uint32 A = ThisIterator ? ThisIterator.GetWord() : 0u;
		const uint32 B = OtherIterator ? OtherIterator.GetWord() : 0u;
		if (A != B)
		{
			// Check if A contains all of the ones in B.
			if ((A & B) != B)
			{
				return false;
			}
		}

		++ThisIterator;
		++OtherIterator;
	}

	return true;
}

void FMediaTileSelection::Include(const FMediaTileSelection& InOther)
{
	ensure(Tiles.Num() == InOther.Tiles.Num());

	Tiles = TBitArray<>::BitwiseOR(Tiles, InOther.Tiles, EBitwiseOperatorFlags::MaxSize);
}

void FMediaTileSelection::SetVisible(int32 InTileCoordX, int32 InTileCoordY)
{
	// Ensure makes caller bugs visible during development; the clamp keeps shipping safe by
	// preventing a negative/oversized coord from indexing out of the underlying TBitArray.
	ensureMsgf(0 <= InTileCoordX && InTileCoordX < Dimensions.X && 0 <= InTileCoordY && InTileCoordY < Dimensions.Y,
		TEXT("FMediaTileSelection::SetVisible out-of-range coord (%d, %d) for dimensions (%d, %d)"),
		InTileCoordX, InTileCoordY, Dimensions.X, Dimensions.Y);
	const int32 ClampedX = FMath::Clamp(InTileCoordX, 0, Dimensions.X - 1);
	const int32 ClampedY = FMath::Clamp(InTileCoordY, 0, Dimensions.Y - 1);
	const int32 Index = ToIndex(ClampedX, ClampedY, Dimensions);
	Tiles[Index] = true;
	bCachedVisibleRegionDirty = true;
}

TArray<FIntPoint> FMediaTileSelection::GetVisibleCoordinates() const
{
	TArray<FIntPoint> OutCoordinates;

	for (int32 CoordY = 0; CoordY < Dimensions.Y; ++CoordY)
	{
		for (int32 CoordX = 0; CoordX < Dimensions.X; ++CoordX)
		{
			if (Tiles[ToIndex(CoordX, CoordY, Dimensions)])
			{
				OutCoordinates.Emplace(CoordX, CoordY);
			}
		}
	}

	return OutCoordinates;
}

TArray<FIntRect> FMediaTileSelection::GetVisibleRegions(const FMediaTileSelection* InCurrentTileSelection) const
{
	/**
	 * This is a two-pass algorithm to batch tiles into contiguous regions, with a bias for row groupings.
	 * First, we iterate through visible tiles, and create regions for (horizontally) contiguous tiles in each row.
	 * Second, we create the final regions out of (vertically) contiguous row regions of matching width & position.
	*/

	TArray<TArray<FIntRect>> RowsOfRegions;

	for (int32 CoordY = 0; CoordY < Dimensions.Y; ++CoordY)
	{
		TArray<FIntRect> RegionsPerRow;
		TBitArray<> PreviousVisibleTiles(0, Dimensions.X);
		for (int32 CoordX = 0; CoordX < Dimensions.X; ++CoordX)
		{
			const int32 TileIndex = ToIndex(CoordX, CoordY, Dimensions);

			bool bOnlyIncludeMissingTiles = InCurrentTileSelection != nullptr;

			// Interpretation: If cached selection doesn't have a tile and CurrentTileSelection (latest) does, then we need to count it as a missing tile.
			const bool bIsThisTileMissing = bOnlyIncludeMissingTiles ? (!Tiles[TileIndex] && InCurrentTileSelection->Tiles[TileIndex]) : (Tiles[TileIndex]);
			if (bIsThisTileMissing)
			{
				FIntPoint TileCoord(CoordX, CoordY);
				PreviousVisibleTiles[CoordX] = true;
				const bool bIsPreviousRowTileVisible = (CoordX > 0) ? PreviousVisibleTiles[CoordX - 1] : false;

				if (bIsPreviousRowTileVisible)
				{
					RegionsPerRow.Last().Include(TileCoord + 1);
				}
				else
				{
					RegionsPerRow.Emplace(TileCoord, TileCoord + 1);
				}
			}
		}

		if (RegionsPerRow.Num() > 0)
		{
			RowsOfRegions.Add(MoveTemp(RegionsPerRow));
		}
	}

	TArray<FIntRect> FinalRegions;

	for (const TArray<FIntRect>& RegionsPerRow : RowsOfRegions)
	{
		for (const FIntRect& Region : RegionsPerRow)
		{
			FIntRect* ContiguousRegion = FinalRegions.FindByPredicate([&Region](const FIntRect& BatchedRegion)
				{
					// Batch row regions if their width matches and if they are vertically contiguous.
					return (Region.Min.X == BatchedRegion.Min.X) && (Region.Max.X == BatchedRegion.Max.X) && (Region.Min.Y == BatchedRegion.Max.Y);
				});

			if (ContiguousRegion != nullptr)
			{
				ContiguousRegion->Max.Y++;
			}
			else
			{
				FinalRegions.Add(Region);
			}
		}
	}

	return FinalRegions;
}

FIntRect FMediaTileSelection::GetVisibleRegion() const
{
	// We offload the region calculation to the loader workers, instead of constantly updating it during SetVisible().
	// Not thread safe, but only accessed sequentially in individual worker thread copies.

	if (bCachedVisibleRegionDirty)
	{
		FIntPoint Min = TNumericLimits<int32>::Max();
		FIntPoint Max = TNumericLimits<int32>::Min();

		for (int32 CoordY = 0; CoordY < Dimensions.Y; ++CoordY)
		{
			for (int32 CoordX = 0; CoordX < Dimensions.X; ++CoordX)
			{
				if (Tiles[ToIndex(CoordX, CoordY, Dimensions)])
				{
					Min.X = FMath::Min(Min.X, CoordX);
					Min.Y = FMath::Min(Min.Y, CoordY);
					Max.X = FMath::Max(Max.X, CoordX);
					Max.Y = FMath::Max(Max.Y, CoordY);
				}
			}
		}

		if (Max.X >= Min.X && Max.Y >= Min.Y)
		{
			CachedVisibleRegion = FIntRect(Min, Max + 1);
		}
		else
		{
			CachedVisibleRegion = FIntRect();
		}

		bCachedVisibleRegionDirty = false;
	}

	return CachedVisibleRegion;
}

int32 FMediaTileSelection::NumVisibleTiles() const
{
	int32 NumVisibleTiles = 0;

	for (int32 CoordY = 0; CoordY < Dimensions.Y; ++CoordY)
	{
		for (int32 CoordX = 0; CoordX < Dimensions.X; ++CoordX)
		{
			if (Tiles[ToIndex(CoordX, CoordY, Dimensions)])
			{
				++NumVisibleTiles;
			}
		}
	}

	return NumVisibleTiles;
}
