// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/MediaTileVisibility.h"

#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"

void FMediaTileVisibilityRequest::Reset()
{
	VisibleTiles.Reset();
	MipLevelToUpscale = -1;
}

void FMediaTileVisibilityRequest::Merge(const FMediaTileVisibilityRequest& InOther)
{
	for (const TPair<int32, FMediaTileSelection>& Pair : InOther.VisibleTiles)
	{
		if (FMediaTileSelection* Existing = VisibleTiles.Find(Pair.Key))
		{
			if (Existing->GetDimensions() == Pair.Value.GetDimensions())
			{
				Existing->Include(Pair.Value);
			}
			else
			{
				// Dimension mismatch is a programming error: providers must agree on
				// per-mip tile counts derived from the sequence info. Keep the larger
				// selection so we don't silently drop coverage.
				const FIntPoint ExistingDim = Existing->GetDimensions();
				const FIntPoint OtherDim = Pair.Value.GetDimensions();
				if (OtherDim.X * OtherDim.Y > ExistingDim.X * ExistingDim.Y)
				{
					*Existing = Pair.Value;
				}
				ensureMsgf(false,
					TEXT("FMediaTileVisibilityRequest::Merge: dimension mismatch at mip %d"),
					Pair.Key);
			}
		}
		else
		{
			VisibleTiles.Add(Pair.Key, Pair.Value);
		}
	}

	if (InOther.MipLevelToUpscale >= 0)
	{
		MipLevelToUpscale = (MipLevelToUpscale >= 0)
			? FMath::Min(MipLevelToUpscale, InOther.MipLevelToUpscale)
			: InOther.MipLevelToUpscale;
	}
}
