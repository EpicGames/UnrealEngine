// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformMediaSource.h"

#include "UObject/ObjectSaveContext.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/MediaFrameWorkObjectVersion.h"
#include "Modules/ModuleManager.h"
#include "IMediaModule.h"
#include "MediaAssetsPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlatformMediaSource)

#if WITH_EDITOR
	#include "Interfaces/ITargetPlatform.h"
#endif


void UPlatformMediaSource::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITORONLY_DATA
	// Do this only if we are cooking (aka: have a target platform)
	const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
	if (TargetPlatform)
	{
		TObjectPtr<UMediaSource>* PlatformMediaSource = PlatformMediaSources.Find(TargetPlatform->IniPlatformName());
		MediaSource = (PlatformMediaSource != nullptr) ? *PlatformMediaSource : nullptr;
	}
#endif
	Super::PreSave(ObjectSaveContext);
}

/* UMediaSource interface
 *****************************************************************************/

FString UPlatformMediaSource::GetUrl() const
{
	// Guard against reentrant calls.
	static bool bIsGettingUrl = false;
	if (bIsGettingUrl)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetUrl - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return FString();
	}
	TGuardValue<bool> GettingUrlGuard(bIsGettingUrl, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	return (PlatformMediaSource != nullptr) ? PlatformMediaSource->GetUrl() : FString();
}


void UPlatformMediaSource::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	Ar.UsingCustomVersion(FMediaFrameworkObjectVersion::GUID);

	auto MediaCustomVersion = Ar.CustomVer(FMediaFrameworkObjectVersion::GUID);
	auto SeqCustomVersion = Ar.CustomVer(FSequencerObjectVersion::GUID);

	if (Ar.IsLoading() && (SeqCustomVersion < FSequencerObjectVersion::RenameMediaSourcePlatformPlayers))
	{
		FString DummyDefaultSource;
		Ar << DummyDefaultSource;

#if WITH_EDITORONLY_DATA
		Ar << PlatformMediaSources;
#endif
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (Ar.IsFilterEditorOnly())
		{
			Ar << MediaSource;
		}
		else
		{
			IMediaModule* MediaModule = static_cast<IMediaModule*>(FModuleManager::Get().GetModule(TEXT("Media")));

			if (Ar.IsLoading() && MediaCustomVersion < FMediaFrameworkObjectVersion::SerializeGUIDsInPlatformMediaSourceInsteadOfPlainNames)
			{
				// Load old data version
				decltype(PlatformMediaSources) OldPlatformMediaSources;

				Ar << OldPlatformMediaSources;

				// Do filter platforms that we cannot translate so they do not pop up visibly
				// (we will loose them on save anyways as we cannot generate a GUID)
				PlatformMediaSources.Empty();
				BlindPlatformMediaSources.Empty();
				if (MediaModule)
				{
					for (auto Entry : OldPlatformMediaSources)
					{
						FGuid PlatformGuid = MediaModule->GetPlatformGuid(FName(Entry.Key));
						if (PlatformGuid.IsValid())
						{
							PlatformMediaSources.Add(Entry);
						}
					}
				}
			}
			else
			{
				// New editor data format
				TMap<FGuid, UMediaSource*> PlatformGuidMediaSources;

				if (Ar.IsSaving())
				{
					if (MediaModule)
					{
						for (auto Entry : PlatformMediaSources)
						{
							FGuid PlatformGuid = MediaModule->GetPlatformGuid(FName(Entry.Key));
							// Check if we got something (just to be sure, we really expect this to work)
							if (PlatformGuid.IsValid())
							{
								PlatformGuidMediaSources.Add(TTuple<FGuid, UMediaSource*>(PlatformGuid, Entry.Value));
							}
						}
					}

					// Move over any blind data we encountered when loading earlier - but avoid overriding any data existing in the object otherwise
					for (auto BlindEntry : BlindPlatformMediaSources)
					{
						PlatformGuidMediaSources.FindOrAdd(BlindEntry.Key, BlindEntry.Value);
					}

					Ar << PlatformGuidMediaSources;
				}
				else if(Ar.IsLoading())
				{
					Ar << PlatformGuidMediaSources;

					PlatformMediaSources.Empty();
					BlindPlatformMediaSources.Empty();
					for (auto Entry : PlatformGuidMediaSources)
					{
						FName PlatformName = MediaModule ? MediaModule->GetPlatformName(Entry.Key) : FName();
						// Could we resolve the GUID or is this an unknown platform?
						if (PlatformName.IsValid())
						{
							PlatformMediaSources.Add(TTuple<FString, UMediaSource*>(PlatformName.ToString(), Entry.Value));
						}
						else
						{
							BlindPlatformMediaSources.Add(Entry);
						}
					}
				}
			}
		}
#else
		Ar << MediaSource;
#endif
	}
}


bool UPlatformMediaSource::Validate() const
{
	// Guard against reentrant calls.
	static bool bIsValidating = false;
	if (bIsValidating)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::Validate - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return false;
	}
	TGuardValue<bool> ValidatingGuard(bIsValidating, true);

#if WITH_EDITORONLY_DATA
	for (auto PlatformNameMediaSourcePair : PlatformMediaSources)
	{
		UMediaSource* PlatformMediaSource = PlatformNameMediaSourcePair.Value;

		if (PlatformMediaSource != nullptr && PlatformMediaSource->Validate() == false)
		{
			return false;
		}
	}

	return PlatformMediaSources.Num() > 0;
#else
	return (MediaSource != nullptr) ? MediaSource->Validate() : false;
#endif
}

#if WITH_EDITOR
FString UPlatformMediaSource::GetDescriptionString() const
{
	// Guard against reentrant calls.
	static bool bIsGettingConfigText = false;
	if (bIsGettingConfigText)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetConfigurationDisplayText - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return FString();
	}
	TGuardValue<bool> Guard(bIsGettingConfigText, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	return (PlatformMediaSource != nullptr) ? PlatformMediaSource->GetDescriptionString() : FString();
}

void UPlatformMediaSource::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	// Guard against reentrant calls.
	static bool bIsGettingInfoElements = false;
	if (bIsGettingInfoElements)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetDetailsPanelInfoElements - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return;
	}
	TGuardValue<bool> Guard(bIsGettingInfoElements, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	if (PlatformMediaSource != nullptr)
	{
		PlatformMediaSource->GetDetailsPanelInfoElements(OutInfoElements);
	}
}
#endif //WITH_EDITOR

/* UPlatformMediaSource implementation
 *****************************************************************************/

UMediaSource* UPlatformMediaSource::GetMediaSource() const
{
#if WITH_EDITORONLY_DATA
	const FString RunningPlatformName(FPlatformProperties::IniPlatformName());
	TObjectPtr<UMediaSource> const* PlatformMediaSource = PlatformMediaSources.Find(RunningPlatformName);

	if (PlatformMediaSource == nullptr)
	{
		return nullptr;
	}

	return *PlatformMediaSource;
#else
	return MediaSource;
#endif
}


/* IMediaOptions interface
 *****************************************************************************/

FName UPlatformMediaSource::GetDesiredPlayerName() const
{
	UMediaSource* PlatformMediaSource = GetMediaSource();

	if (PlatformMediaSource != nullptr)
	{
		return PlatformMediaSource->GetDesiredPlayerName();
	}

	return Super::GetDesiredPlayerName();
}


bool UPlatformMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	// Guard against reentrant calls.
	static bool bIsGettingOption = false;
	if (bIsGettingOption)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bIsGettingOption, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	
	if (PlatformMediaSource != nullptr)
	{
		return PlatformMediaSource->GetMediaOption(Key, DefaultValue);
	}
	
	return Super::GetMediaOption(Key, DefaultValue);
}


double UPlatformMediaSource::GetMediaOption(const FName& Key, double DefaultValue) const
{
	// Guard against reentrant calls.
	static bool bIsGettingOption = false;
	if (bIsGettingOption)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bIsGettingOption, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	
	if (PlatformMediaSource != nullptr)
	{
		return PlatformMediaSource->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


int64 UPlatformMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	// Guard against reentrant calls.
	static bool bIsGettingOption = false;
	if (bIsGettingOption)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bIsGettingOption, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	
	if (PlatformMediaSource != nullptr)
	{
		return PlatformMediaSource->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


FString UPlatformMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	// Guard against reentrant calls.
	static bool bIsGettingOption = false;
	if (bIsGettingOption)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bIsGettingOption, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	
	if (PlatformMediaSource != nullptr)
	{
		return PlatformMediaSource->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


FText UPlatformMediaSource::GetMediaOption(const FName& Key, const FText& DefaultValue) const
{
	// Guard against reentrant calls.
	static bool bIsGettingOption = false;
	if (bIsGettingOption)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bIsGettingOption, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	
	if (PlatformMediaSource != nullptr)
	{
		return PlatformMediaSource->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


bool UPlatformMediaSource::HasMediaOption(const FName& Key) const
{
	// Guard against reentrant calls.
	static bool bIsHasOption = false;
	if (bIsHasOption)
	{
		UE_LOGF(LogMediaAssets, Warning, "UPlatformMediaSource::HasMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return false;
	}
	TGuardValue<bool> HasOptionGuard(bIsHasOption, true);

	UMediaSource* PlatformMediaSource = GetMediaSource();
	
	if (PlatformMediaSource != nullptr)
	{
		return PlatformMediaSource->HasMediaOption(Key);
	}

	return Super::HasMediaOption(Key);
}

