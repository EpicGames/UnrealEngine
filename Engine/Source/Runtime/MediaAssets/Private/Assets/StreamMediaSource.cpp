// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StreamMediaSource)


/* UMediaSource overrides
 *****************************************************************************/

FString UStreamMediaSource::GetUrl() const
{
	return StreamUrl;
}


bool UStreamMediaSource::Validate() const
{
	return StreamUrl.Contains(TEXT("://"));
}

#if WITH_EDITOR

void UStreamMediaSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamMediaSource, StreamUrl))
	{
		GenerateThumbnail();
	}
}

#endif
