// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSynchronizableMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimeSynchronizableMediaSource)



UTimeSynchronizableMediaSource::UTimeSynchronizableMediaSource()
	: bUseTimeSynchronization(false)
	, FrameDelay(0)
	, TimeDelay(0.0)
{ }


/*
* IMediaOptions interface
*/

bool UTimeSynchronizableMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == TimeSynchronizableMedia::UseTimeSynchronizatioOption)
	{
		return bUseTimeSynchronization;
	}
	
	if (Key == TimeSynchronizableMedia::AutoDetect)
	{
		return bAutoDetectInput;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

int64 UTimeSynchronizableMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == TimeSynchronizableMedia::FrameDelay)
	{
		return FMath::Clamp(FrameDelay, 0, 4);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

double UTimeSynchronizableMediaSource::GetMediaOption(const FName& Key, double DefaultValue) const
{
	if (Key == TimeSynchronizableMedia::TimeDelay)
	{
		return TimeDelay;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

FString UTimeSynchronizableMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	return Super::GetMediaOption(Key, DefaultValue);
}

bool UTimeSynchronizableMediaSource::HasMediaOption(const FName& Key) const
{
	if (Key == TimeSynchronizableMedia::UseTimeSynchronizatioOption
		|| Key == TimeSynchronizableMedia::FrameDelay
		|| Key == TimeSynchronizableMedia::TimeDelay
		|| Key == TimeSynchronizableMedia::AutoDetect)
	{
		return true;
	}

	return Super::HasMediaOption(Key);
}


