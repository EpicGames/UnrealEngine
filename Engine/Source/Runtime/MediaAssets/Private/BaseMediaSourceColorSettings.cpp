// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseMediaSourceColorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseMediaSourceColorSettings)

#if WITH_EDITOR
void FMediaSourceColorSettings::UpdateColorSpaceChromaticities()
{
	if (ColorSpaceOverride != ETextureColorSpace::TCS_Custom)
	{
		UE::Color::FColorSpace Chromaticities(static_cast<UE::Color::EColorSpace>(ColorSpaceOverride));
		Chromaticities.GetChromaticities(RedChromaticityCoordinate, GreenChromaticityCoordinate, BlueChromaticityCoordinate, WhiteChromaticityCoordinate);
	}
}
#endif // WITH_EDITOR

FNativeMediaSourceColorSettings::FNativeMediaSourceColorSettings()
	: EncodingOverride(UE::Color::EEncoding::None)
	, ReferenceWhiteOverride(UE::Color::EReferenceWhite::None)
	, ColorSpaceOverride()
	, ChromaticAdaptationMethod(UE::Color::DEFAULT_CHROMATIC_ADAPTATION_METHOD)
{}

FNativeMediaSourceColorSettings::~FNativeMediaSourceColorSettings() = default;

FNativeMediaSourceColorSettings::FNativeMediaSourceColorSettings(const FNativeMediaSourceColorSettings& Other)
{
	EncodingOverride.store(Other.EncodingOverride);
	ReferenceWhiteOverride.store(Other.ReferenceWhiteOverride);
	ChromaticAdaptationMethod.store(Other.ChromaticAdaptationMethod);

	FScopeLock Lock(&Other.ColorSpaceCriticalSection);
	ColorSpaceOverride = Other.ColorSpaceOverride;
}

FNativeMediaSourceColorSettings& FNativeMediaSourceColorSettings::operator=(const FNativeMediaSourceColorSettings& Other)
{
	if (this != &Other)
	{
		EncodingOverride.store(Other.EncodingOverride);
		ReferenceWhiteOverride.store(Other.ReferenceWhiteOverride);
		ChromaticAdaptationMethod.store(Other.ChromaticAdaptationMethod);

		FScopeLock Lock(&ColorSpaceCriticalSection);
		FScopeLock LockOther(&Other.ColorSpaceCriticalSection);
		ColorSpaceOverride = Other.ColorSpaceOverride;
	}
	return *this;
}

void FNativeMediaSourceColorSettings::Update(const FMediaSourceColorSettings& InSettings)
{
	EncodingOverride = static_cast<UE::Color::EEncoding>(InSettings.EncodingOverride);
	ReferenceWhiteOverride = static_cast<UE::Color::EReferenceWhite>(InSettings.ReferenceWhiteOverride);
	ChromaticAdaptationMethod = static_cast<UE::Color::EChromaticAdaptationMethod>(InSettings.ChromaticAdaptationMethod);

	FScopeLock Lock(&ColorSpaceCriticalSection);

	if (InSettings.ColorSpaceOverride == ETextureColorSpace::TCS_Custom)
	{
		ColorSpaceOverride = UE::Color::FColorSpace(
			InSettings.RedChromaticityCoordinate,
			InSettings.GreenChromaticityCoordinate,
			InSettings.BlueChromaticityCoordinate,
			InSettings.WhiteChromaticityCoordinate
		);
	}
	else if (InSettings.ColorSpaceOverride != ETextureColorSpace::TCS_None)
	{
		ColorSpaceOverride = UE::Color::FColorSpace(static_cast<UE::Color::EColorSpace>(InSettings.ColorSpaceOverride));
	}
	else
	{
		ColorSpaceOverride.Reset();
	}
}

bool FNativeMediaSourceColorSettings::HasColorSpaceOverride() const
{
	FScopeLock Lock(&ColorSpaceCriticalSection);

	return ColorSpaceOverride.IsSet();
}

const UE::Color::FColorSpace& FNativeMediaSourceColorSettings::GetColorSpaceOverride(const UE::Color::FColorSpace& InDefaultColorSpace) const
{
	FScopeLock Lock(&ColorSpaceCriticalSection);
	
	if (ColorSpaceOverride.IsSet())
	{
		return ColorSpaceOverride.GetValue();
	}

	return InDefaultColorSpace;
}

void FNativeMediaSourceColorSettings::SetColorSpaceOverride(const UE::Color::FColorSpace& InColorSpaceOverride)
{
	FScopeLock Lock(&ColorSpaceCriticalSection);

	ColorSpaceOverride = InColorSpaceOverride;
}

