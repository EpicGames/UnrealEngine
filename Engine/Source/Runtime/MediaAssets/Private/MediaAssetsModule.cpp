// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaAssetsPrivate.h"

#include "CoreMinimal.h"
#include "FileMediaSource.h"
#include "IMediaAssetsModule.h"
#include "MediaSourceRendererInterface.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#include "MediaPlayer.h"


DEFINE_LOG_CATEGORY(LogMediaAssets);


/**
 * Implements the MediaAssets module.
 */
class FMediaAssetsModule
	: public FSelfRegisteringExec
	, public IMediaAssetsModule
{
protected:

	//~ FSelfRegisteringExec interface

	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("MEDIA")))
		{
			FString MovieCmd = FParse::Token(Cmd, 0);

			if (MovieCmd.Contains(TEXT("PLAY")))
			{
				for (TObjectIterator<UMediaPlayer> It; It; ++It)
				{
					UMediaPlayer* MediaPlayer = *It;
					MediaPlayer->Play();
				}
			}
			else if (MovieCmd.Contains(TEXT("PAUSE")))
			{
				for (TObjectIterator<UMediaPlayer> It; It; ++It)
				{
					UMediaPlayer* MediaPlayer = *It;
					MediaPlayer->Pause();
				}
			}

			return true;
		}

		return false;
	}

public:
	//~ IMediaAssetsModule interface
	virtual int32 RegisterGetPlayerFromObject(const FOnGetPlayerFromObject& Delegate) override
	{
		int32 Index = GetPlayerFromObjectDelegates.Num();
		GetPlayerFromObjectDelegates.Add(Delegate);
		return Index;
	}

	virtual void UnregisterGetPlayerFromObject(int32 DelegateID) override
	{
		GetPlayerFromObjectDelegates[DelegateID].Unbind();
	}

	virtual UMediaPlayer* GetPlayerFromObject(UObject* Object, UObject*& PlayerProxy) override
	{
		UMediaPlayer* Player = nullptr;
		PlayerProxy = nullptr;

		// Go through all our delegates.
		for (FOnGetPlayerFromObject& Delegate : GetPlayerFromObjectDelegates)
		{
			if (Delegate.IsBound())
			{
				Player = Delegate.Execute(Object, PlayerProxy);
				if (Player != nullptr)
				{
					break;
				}
			}
		}

		return Player;
	}

	virtual void RegisterCreateMediaSourceRenderer(const FOnCreateMediaSourceRenderer& Delegate) override
	{
		CreateMediaSourceRendererDelegate = Delegate;
	}

	virtual void UnregisterCreateMediaSourceRenderer()
	{
		CreateMediaSourceRendererDelegate.Unbind();
	}

	virtual FDelegateHandle RegisterOnMediaStateChangedEvent(FMediaStateChangedDelegate::FDelegate InStateChangedDelegate) override 
	{ 
		return OnMediaStateChanged.Add(InStateChangedDelegate);
	}

	virtual void UnregisterOnMediaStateChangedEvent(FDelegateHandle InHandle) override 
	{
		OnMediaStateChanged.Remove(InHandle);
	}

	virtual void BroadcastOnMediaStateChangedEvent(const TArray<FString>& InActorsPathNames, uint8 EnumState, bool bRemoteBroadcast = false) override
	{
		OnMediaStateChanged.Broadcast(InActorsPathNames, EnumState, bRemoteBroadcast);
	}

	virtual UObject* CreateMediaSourceRenderer() override
	{
		UObject* Renderer = nullptr;
		if (CreateMediaSourceRendererDelegate.IsBound())
		{
			Renderer = CreateMediaSourceRendererDelegate.Execute();
		}
		return Renderer;
	}
	
	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// Register media source spawners.
		FMediaSourceSpawnDelegate SpawnDelegate =
			FMediaSourceSpawnDelegate::CreateStatic(&FMediaAssetsModule::SpawnMediaSourceForString);
		for (const FString& Ext : FileExtensions)
		{
			UMediaSource::RegisterSpawnFromFileExtension(Ext, SpawnDelegate);
		}
	}

	virtual void ShutdownModule() override
	{
		// Unregister media source spawners.
		for (const FString& Ext : FileExtensions)
		{
			UMediaSource::UnregisterSpawnFromFileExtension(Ext);
		}
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

private:
	/**
	 * Creates a media source for MediaPath.
	 *
	 * @param	MediaPath		File path to the media.
	 * @param	Outer			Outer to use for this object.
	 */
	static UMediaSource* SpawnMediaSourceForString(const FString& MediaPath, UObject* Outer)
	{
		TObjectPtr<UFileMediaSource> MediaSource =
			NewObject<UFileMediaSource>(Outer, NAME_None, RF_Transactional);
		MediaSource->SetFilePath(MediaPath);

		return MediaSource;
	}

	/** All the functions that can get a player from an object. */
	TArray<FOnGetPlayerFromObject> GetPlayerFromObjectDelegates;

	/** List of file extensions that we support. */
	const TArray<FString> FileExtensions =
	{
		TEXT("mov"),
		TEXT("mp4"),
		TEXT("wmv"),
	};

	/** Delegate to create an object that implements IMediaSourceRendererInterface. */
	FOnCreateMediaSourceRenderer CreateMediaSourceRendererDelegate;

	/** Delegate that reacts to change in Media state (Play, Stop, Pause etc.) */
	FMediaStateChangedDelegate OnMediaStateChanged;
};


IMPLEMENT_MODULE(FMediaAssetsModule, MediaAssets);
