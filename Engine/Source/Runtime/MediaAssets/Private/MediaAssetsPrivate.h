// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Stats/Stats.h"


/** Declares a log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogMediaAssets, Log, All);
