// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSourcePathValidation.h"

#include "Containers/StringView.h"
#include "HAL/IConsoleManager.h"


namespace UE::MediaAssets
{
	static TAutoConsoleVariable<bool> CVarAllowNetworkPaths(
		TEXT("MediaAssets.AllowNetworkPaths"),
		true,
		TEXT("If true (default), media sources may resolve UNC / network paths in their file/sequence path properties. ")
		TEXT("Disable on online game servers and untrusted-content workflows to harden against UNC path injection through ")
		TEXT("media-source assets. ")
		TEXT("Read-only: must be set in DefaultEngine.ini or on the command line before the MediaAssets module loads; ")
		TEXT("changing this value at runtime has no effect. ")
		TEXT("Detection is path-string based and Windows-focused (\\\\server, //server, \\\\?\\UNC\\, \\\\.\\UNC\\, file:// URIs); ")
		TEXT("mapped network drives (e.g. Z:\\ from 'net use') and non-Windows network mounts (/Volumes on macOS, ")
		TEXT("arbitrary Linux mount points) cannot be detected from the path string alone."),
		ECVF_ReadOnly);

	bool IsUNCPath(FStringView InPath)
	{
		// Strip a leading file:// scheme so wrapped URIs don't bypass the prefix check.
		constexpr FStringView FileScheme = TEXTVIEW("file://");
		if (InPath.StartsWith(FileScheme, ESearchCase::IgnoreCase))
		{
			InPath = InPath.RightChop(FileScheme.Len());
		}

		// Forward-slash UNC: //server/share (no namespace ambiguity in this form).
		if (InPath.StartsWith(TEXTVIEW("//")))
		{
			return true;
		}

		// Backslash forms: \\?\ and \\.\ are Win32 namespace prefixes that are UNC only when followed by UNC\.
		if (InPath.StartsWith(TEXTVIEW("\\\\")))
		{
			if (InPath.Len() >= 3 && (InPath[2] == TEXT('?') || InPath[2] == TEXT('.')))
			{
				return InPath.StartsWith(TEXTVIEW("\\\\?\\UNC\\"), ESearchCase::IgnoreCase)
					|| InPath.StartsWith(TEXTVIEW("\\\\.\\UNC\\"), ESearchCase::IgnoreCase);
			}
			return true;
		}

		return false;
	}

	bool AreNetworkPathsAllowed()
	{
		return CVarAllowNetworkPaths.GetValueOnAnyThread();
	}

	bool IsSourcePathAllowed(FStringView InPath)
	{
		return AreNetworkPathsAllowed() || !IsUNCPath(InPath);
	}
}
