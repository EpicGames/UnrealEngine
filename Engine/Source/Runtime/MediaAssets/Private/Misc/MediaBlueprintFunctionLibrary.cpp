// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MediaBlueprintFunctionLibrary.h"

#include "MediaCaptureSupport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaBlueprintFunctionLibrary)


/* UMediaBlueprintFunctionLibrary interface
 *****************************************************************************/

void UMediaBlueprintFunctionLibrary::EnumerateAudioCaptureDevices(TArray<FMediaCaptureDevice>& OutDevices, int32 Filter)
{
	const auto FilterEnum = (EMediaAudioCaptureDeviceFilter)Filter;

	TArray<FMediaCaptureDeviceInfo> DeviceInfos;
	MediaCaptureSupport::EnumerateAudioCaptureDevices(DeviceInfos);

	for (const auto& DeviceInfo : DeviceInfos)
	{
		if (((DeviceInfo.Type == EMediaCaptureDeviceType::Audio) && EnumHasAnyFlags(FilterEnum, EMediaAudioCaptureDeviceFilter::Unknown)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::AudioCard) && EnumHasAnyFlags(FilterEnum, EMediaAudioCaptureDeviceFilter::Card)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::AudioSoftware) && EnumHasAnyFlags(FilterEnum, EMediaAudioCaptureDeviceFilter::Software)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::Microphone) && EnumHasAnyFlags(FilterEnum, EMediaAudioCaptureDeviceFilter::Microphone)))
		{
			OutDevices.Add(FMediaCaptureDevice(DeviceInfo.DisplayName, DeviceInfo.Url));
		}
	}
}


void UMediaBlueprintFunctionLibrary::EnumerateVideoCaptureDevices(TArray<FMediaCaptureDevice>& OutDevices, int32 Filter)
{
	const auto FilterEnum = (EMediaVideoCaptureDeviceFilter)Filter;

	TArray<FMediaCaptureDeviceInfo> DeviceInfos;
	MediaCaptureSupport::EnumerateVideoCaptureDevices(DeviceInfos);

	for (const auto& DeviceInfo : DeviceInfos)
	{
		if (((DeviceInfo.Type == EMediaCaptureDeviceType::DepthSensor) && EnumHasAnyFlags(FilterEnum, EMediaVideoCaptureDeviceFilter::Webcam)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::Video) && EnumHasAnyFlags(FilterEnum, EMediaVideoCaptureDeviceFilter::Unknown)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::VideoCard) && EnumHasAnyFlags(FilterEnum, EMediaVideoCaptureDeviceFilter::Card)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::VideoSoftware) && EnumHasAnyFlags(FilterEnum, EMediaVideoCaptureDeviceFilter::Software)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::Webcam) && EnumHasAnyFlags(FilterEnum, EMediaVideoCaptureDeviceFilter::Webcam)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::WebcamFront) && EnumHasAnyFlags(FilterEnum, EMediaVideoCaptureDeviceFilter::Webcam)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::WebcamRear) && EnumHasAnyFlags(FilterEnum, EMediaVideoCaptureDeviceFilter::Webcam)))
		{
			OutDevices.Add(FMediaCaptureDevice(DeviceInfo.DisplayName, DeviceInfo.Url));
		}
	}
}


void UMediaBlueprintFunctionLibrary::EnumerateWebcamCaptureDevices(TArray<FMediaCaptureDevice>& OutDevices, int32 Filter)
{
	const auto FilterEnum = (EMediaWebcamCaptureDeviceFilter)Filter;

	TArray<FMediaCaptureDeviceInfo> DeviceInfos;
	MediaCaptureSupport::EnumerateVideoCaptureDevices(DeviceInfos);

	for (const auto& DeviceInfo : DeviceInfos)
	{
		if (((DeviceInfo.Type == EMediaCaptureDeviceType::DepthSensor) && EnumHasAnyFlags(FilterEnum, EMediaWebcamCaptureDeviceFilter::DepthSensor)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::Webcam) && EnumHasAnyFlags(FilterEnum, EMediaWebcamCaptureDeviceFilter::Unknown)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::WebcamFront) && EnumHasAnyFlags(FilterEnum, EMediaWebcamCaptureDeviceFilter::Front)) ||
			((DeviceInfo.Type == EMediaCaptureDeviceType::WebcamRear) && EnumHasAnyFlags(FilterEnum, EMediaWebcamCaptureDeviceFilter::Rear)))
		{
			OutDevices.Add(FMediaCaptureDevice(DeviceInfo.DisplayName, DeviceInfo.Url));
		}
	}
}

