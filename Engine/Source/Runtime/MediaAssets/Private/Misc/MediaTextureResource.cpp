// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MediaTextureResource.h"
#include "MediaAssetsPrivate.h"
#include "Modules/ModuleManager.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "ExternalTexture.h"
#include "IMediaModule.h"
#include "IMediaClock.h"
#include "IMediaPlayer.h"
#include "IMediaSamples.h"
#include "IMediaTextureSample.h"
#include "IMediaTextureSampleConverter.h"
#include "MediaDelegates.h"
#include "MediaPlayerFacade.h"
#include "MediaSampleSource.h"
#include "MediaShaders.h"
#include "PipelineStateCache.h"
#include "RenderCaptureInterface.h"
#include "RenderGraphUtils.h"
#include "SceneUtils.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "RenderUtils.h"
#include "RHIStaticStates.h"
#include "RHIUtilities.h"
#include "GenerateMips.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "Async/Async.h"


#include "HDRHelper.h"

#include "MediaTexture.h"

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
static int32 GMediaCaptureTexture = 0;
static FAutoConsoleVariableRef CVarMediaTextureTriggerCapture(
	TEXT("r.Media.CaptureTexture"),
	GMediaCaptureTexture,
	TEXT("Trigger a GPU capture for the media texture resource update (debug/development builds only).")
);
#endif

#if PLATFORM_ANDROID
# include "Android/AndroidPlatformMisc.h"
#endif

/** Time spent in media player facade closing media. */
DECLARE_CYCLE_STAT(TEXT("MediaAssets MediaTextureResource Render"), STAT_MediaAssets_MediaTextureResourceRender, STATGROUP_Media);

/** Sample time of texture last rendered. */
DECLARE_FLOAT_COUNTER_STAT(TEXT("MediaAssets MediaTextureResource Sample"), STAT_MediaUtils_TextureSampleTime, STATGROUP_Media);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(MEDIA_API, MediaStreaming);

DECLARE_GPU_STAT_NAMED(MediaTextureResource, TEXT("MediaTextureResource"));

static const FMatrix44f SplatMtx = { {1.0f, 0.0f, 0.0f, 0.0f}, {1.0f, 0.0f, 0.0f, 0.0f}, {1.0f, 0.0f, 0.0f, 0.0f}, {1.0f, 0.0f, 0.0f, 0.0f} };

namespace UE::MediaTexture::Private
{
	/** Calculate the maximum number of mips for the given texture dimension in pixels. */
	static int32 CalcMaxNumMips(const FIntPoint& InDim)
	{
		const int32 MaxDimension = FMath::Max(InDim.X, InDim.Y);

		// FloorLog2NonZero requires a positive non-zero value.
		if (MaxDimension <= 0)
		{
			return 1;
		}

		const uint32 NumMips = 1u + FMath::FloorLog2NonZero(static_cast<uint32>(MaxDimension));
		return static_cast<int32>(NumMips);
	}

	uint8 ClampNumMips(uint8 InNumMips, const FIntPoint& InDim)
	{
		if (InNumMips > 1u)
		{
			const uint8 MaxNumMips = static_cast<uint8>(FMath::Min(GMaxTextureMipCount, CalcMaxNumMips(InDim)));
			InNumMips = FMath::Min(InNumMips, MaxNumMips);
		}

		return InNumMips;
	}

	uint8 CalcGeneratedNumMips(const IMediaTextureSample& InSample, bool bInEnableGenMips)
	{
		if (bInEnableGenMips && !MediaTextureSampleFormat::IsBlockCompressedFormat(InSample.GetFormat()))
		{
			return ClampNumMips(static_cast<uint8>(GMaxTextureMipCount), InSample.GetDim());
		}

		return 1u;
	}

	uint8 CalcTextureNumMips(const IMediaTextureSample& InSample, bool bInEnableGenMips)
	{
		const uint8 SampleNumMips = InSample.GetNumMips();

		return SampleNumMips > 1u ? SampleNumMips : CalcGeneratedNumMips(InSample, bInEnableGenMips);
	}
}

/* GPU data deletion helper
 *****************************************************************************/

template<typename ObjectRefType> struct TGPUsyncedDataDeleter : public IMediaClockSink
{
	static TSharedRef<TGPUsyncedDataDeleter<ObjectRefType>, ESPMode::ThreadSafe> Create()
	{
		auto Ret = MakeShared<TGPUsyncedDataDeleter<ObjectRefType>, ESPMode::ThreadSafe>();
		Ret->WeakThis = Ret;
		return Ret;
	}

	virtual ~TGPUsyncedDataDeleter()
	{
		// See if all samples are ready to be retired now...
		if (!Update())
		{
			// They are. No need for any async task...
			return;
		}

		// Start having us ticked so we can retire the rest over the next game loop iterations...
		IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>(TEXT("MediaModule"));
		if (MediaModule)
		{
			MediaModule->GetClock().AddSink(StaticCastSharedRef<IMediaClockSink, TGPUsyncedDataDeleter<ObjectRefType>, ESPMode::ThreadSafe>(WeakThis.Pin().ToSharedRef()));
		}
	}

	void Retire(FRHICommandListImmediate& RHICmdList, const ObjectRefType& Object)
	{
		// Prep "retirement package"
		FRetiringObjectInfo Info;
		Info.Object = Object;
		Info.GPUFence = RHICreateGPUFence(TEXT("MediaTextureResourceReuseFence"));
		Info.RetireTime = FPlatformTime::Seconds();

		// Insert fence. We assume that GPU-workload-wise this marks the spot usage of the sample is done
		RHICmdList.WriteGPUFence(Info.GPUFence);

		// Recall for later checking...
		FScopeLock Lock(&CS);
		Objects.Push(Info);
	}

	bool Update()
	{
		FScopeLock Lock(&CS);

		// Check for any retired samples that are not done being touched by the GPU...
		int32 Idx = 0;
		for (; Idx < Objects.Num(); ++Idx)
		{
			// Either no fence present or the fence has been signaled?
			if (Objects[Idx].GPUFence.IsValid() && !Objects[Idx].GPUFence->Poll())
			{
				// No. This one is still busy, we can stop...
				break;
			}
		}
		// Remove (hence return to the pool / free up fence) all the finished ones...
		if (Idx != 0)
		{
			Objects.RemoveAt(0, Idx);
		}
		return Objects.Num() != 0;
	}

private:
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override
	{
		// Check what data is still not signaled...
		if (!Update())
		{
			// All is gone! Remove reference to this (which will delete this instance)
			IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>(TEXT("MediaModule"));
			if (MediaModule)
			{
				MediaModule->GetClock().RemoveSink(StaticCastSharedRef<IMediaClockSink, TGPUsyncedDataDeleter<ObjectRefType>, ESPMode::ThreadSafe>(WeakThis.Pin().ToSharedRef()));
			}
		}
	}

	struct FRetiringObjectInfo
	{
		ObjectRefType Object;
		FGPUFenceRHIRef GPUFence;
		double RetireTime;
	};

	TWeakPtr<TGPUsyncedDataDeleter<ObjectRefType>, ESPMode::ThreadSafe> WeakThis;
	TArray<FRetiringObjectInfo> Objects;
	FCriticalSection CS;
};

struct FPriorSamples : public TGPUsyncedDataDeleter<TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>>
{
	using BaseType = TGPUsyncedDataDeleter<TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>>;

	static TSharedRef<FPriorSamples, ESPMode::ThreadSafe> Create()
	{
		return StaticCastSharedRef<FPriorSamples, BaseType, ESPMode::ThreadSafe>(BaseType::Create());
	}
};

/* Local helpers
 *****************************************************************************/

namespace MediaTextureResourceHelpers
{
	/**
	 * Get the pixel format for a given sample.
	 *
	 * @param Sample The sample.
	 * @return The sample's pixel format.
	 */
	EPixelFormat GetPixelFormat(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
	{
		switch (Sample->GetFormat())
		{
		case EMediaTextureSampleFormat::CharAYUV:
		case EMediaTextureSampleFormat::CharBGRA:
		case EMediaTextureSampleFormat::CharBMP:
		case EMediaTextureSampleFormat::CharUYVY:
		case EMediaTextureSampleFormat::CharYUY2:
		case EMediaTextureSampleFormat::Char2VUY:
		case EMediaTextureSampleFormat::CharYVYU:
			return PF_B8G8R8A8;

		case EMediaTextureSampleFormat::CharRGBA:
			return PF_R8G8B8A8;

		case EMediaTextureSampleFormat::CharNV12:
		case EMediaTextureSampleFormat::CharNV21:
			return PF_G8; // note: right now this case will be encountered only if CPU-side data in NV12/21 format is in sample -> in this case we cannot create a true NV12 texture OR the platforms view it as U8s anyway 

		case EMediaTextureSampleFormat::FloatRGB:
			return PF_FloatRGB;

		case EMediaTextureSampleFormat::FloatRGBA:
			return PF_FloatRGBA;

		case EMediaTextureSampleFormat::CharBGR10A2:
			return PF_A2B10G10R10;

		case EMediaTextureSampleFormat::ABGR16:
			return PF_A16B16G16R16;

		case EMediaTextureSampleFormat::RGBA16:
			return PF_R16G16B16A16_UNORM;

		case EMediaTextureSampleFormat::ARGB16_BIG:
			return PF_R16G16B16A16_UINT;

		case EMediaTextureSampleFormat::YUVv210:
			return PF_A2B10G10R10;

		case EMediaTextureSampleFormat::YUVv216:
		case EMediaTextureSampleFormat::ShortYUY2:
			return PF_A16B16G16R16;

		case EMediaTextureSampleFormat::Y416:
			return PF_A16B16G16R16;

		case EMediaTextureSampleFormat::R4FL:
			return PF_A32B32G32R32F;

		case EMediaTextureSampleFormat::P010:
			return PF_G16; // note: right now this case will be encountered only if CPU-side data in NV12/21 format is in sample -> in this case we cannot create a true P010 texture OR the platforms view it as U16s anyway 

		case EMediaTextureSampleFormat::P010_RGB1010102:
			return PF_A2B10G10R10;

		case EMediaTextureSampleFormat::DXT1:
			return PF_DXT1;

		case EMediaTextureSampleFormat::DXT5:
			return PF_DXT5;

		case EMediaTextureSampleFormat::BC4:
			return PF_BC4;

		case EMediaTextureSampleFormat::YCoCg_DXT5:
			return PF_DXT5;

		default:
			return PF_Unknown;
		}
	}

	EPixelFormat GetConvertedPixelFormat(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
	{
		switch (Sample->GetFormat())
		{
		case EMediaTextureSampleFormat::CharBGR10A2:
		case EMediaTextureSampleFormat::YUVv210:
		{
			if (Sample->GetEncodingType() != UE::Color::EEncoding::Linear)
			{
				return PF_FloatRGB;
			}
			return PF_A2B10G10R10;
		}

		case EMediaTextureSampleFormat::FloatRGB:
		case EMediaTextureSampleFormat::P010:
		case EMediaTextureSampleFormat::YUVv216:
		case EMediaTextureSampleFormat::ShortYUY2:
			return PF_FloatRGB;

		case EMediaTextureSampleFormat::Y416:
		case EMediaTextureSampleFormat::R4FL:
		case EMediaTextureSampleFormat::FloatRGBA:
		case EMediaTextureSampleFormat::P010_RGB1010102:
			return PF_FloatRGBA;

		case EMediaTextureSampleFormat::BC4:
			return PF_A8;

		case EMediaTextureSampleFormat::RGBA16:
			return PF_R16G16B16A16_UNORM;

		case EMediaTextureSampleFormat::ABGR16:
		case EMediaTextureSampleFormat::ARGB16_BIG:
			return PF_A16B16G16R16;

		// Everything else maps to 8-bit RGB if it is linear or sRGB (we use sRGB encoding in HW). Otherwise: half floats...
		default:
		{
			if (Sample->GetEncodingType() != UE::Color::EEncoding::Linear && Sample->GetEncodingType() != UE::Color::EEncoding::sRGB)
			{
				return PF_FloatRGB;
			}
			return PF_B8G8R8A8;
		}
		}
	}

	/**
	 * Helper function to calculate the stride of the sample buffer
	 * using mip0's stride up to the point where there is 1 pixel left.
	 */
	uint32 CalcSampleStrideForMipLevel(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& InSample, uint8 InMipLevel)
	{
		const uint32 MipStride = InSample->GetStride() >> InMipLevel;
		if (MipStride > 0)
		{
			return MipStride;
		}
		
		// Calculate the smallest stride for this format (for 1 pixel).
		const EPixelFormat PixelFormat = GetPixelFormat(InSample);
		const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[PixelFormat];
		return PixelFormatInfo.GetBlockCountForWidth(1) * PixelFormatInfo.BlockBytes;
	}

	/**
	 * Updates the destination RHI texture from the sample's memory buffers. 
	 */
	void UpdateTextureFromSample(
		FRHICommandListImmediate& InRhiCmdList,
		const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& InSample,
		uint8 InNumMips,
		const TRefCountPtr<FRHITexture>& InDestTexture)
	{
		const uint8* Data = (const uint8*)InSample->GetBuffer();
		check(Data);
		const FPixelFormatInfo& FormatInfo = GPixelFormats[GetPixelFormat(InSample)];
		const FIntPoint SampleDim = InSample->GetDim();

		for (uint8 MipLevel = 0; MipLevel < InNumMips; ++MipLevel)
		{
			const uint32 Width = FMath::Max(1, SampleDim.X >> MipLevel);
			const uint32 Height = FMath::Max(1, SampleDim.Y >> MipLevel);
			const uint32 Stride = CalcSampleStrideForMipLevel(InSample, MipLevel);

			// Don't let a zero stride go through rhi, it causes a delayed error that is hard to track.
			if (!ensure(Stride != 0))
			{
				break;
			}

			FUpdateTextureRegion2D Region(0, 0, 0, 0, Width, Height);
			InRhiCmdList.UpdateTexture2D(InDestTexture, MipLevel, Region, Stride, Data);

			const uint32 HeightUnit = (Height + FormatInfo.BlockSizeY - 1) / FormatInfo.BlockSizeY;
			Data += Stride * HeightUnit;
		}
	}

	bool RequiresSrgbTexture(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
	{
		return Sample->IsOutputSrgb();
	}

	bool RequiresUAVTexture(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample, uint8 NumMips)
	{	
		//UAV output is needed if mips are required and uses CS to generate
		//or if sample converter asks for it
		const IMediaTextureSampleConverter* Converter = Sample->GetMediaTextureSampleConverter();

		bool bNeedsUAV = (Converter && ((Converter->GetConverterInfoFlags() & IMediaTextureSampleConverter::ConverterInfoFlags_NeedUAVOutputTexture) != 0));

		if (NumMips > 1)
		{
			EPixelFormat Format = MediaTextureResourceHelpers::GetConvertedPixelFormat(Sample);
			bNeedsUAV |= FGenerateMips::WillFormatSupportCompute(Format);
		}

		return bNeedsUAV;
	}


	bool RequiresSrgbInputTexture(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
	{
		if (!Sample->IsOutputSrgb())
		{
			return false;
		}
		/*
		* Input textures created to receive CPU side buffer sample data are ONLY created with sRGB attributes
		* if we have any 8-bit RGB(A) format. Any YUV (etc.) format does the conversion in SW when applicable in the
		* conversion process!
		*/
		EMediaTextureSampleFormat Fmt = Sample->GetFormat();
		return Fmt == EMediaTextureSampleFormat::CharBGRA ||
			Fmt == EMediaTextureSampleFormat::CharRGBA ||
			Fmt == EMediaTextureSampleFormat::CharBMP ||
			Fmt == EMediaTextureSampleFormat::DXT1 ||
			Fmt == EMediaTextureSampleFormat::DXT5;
	}

} //namespace

/* FMediaTextureResource structors
 *****************************************************************************/

FMediaTextureResource::FMediaTextureResource(UMediaTexture& InOwner, FIntPoint& InOwnerDim, SIZE_T& InOwnerSize, FLinearColor InClearColor, FGuid InTextureGuid, bool InEnableGenMips, uint8 InNumMips, UE::Color::EColorSpace OverrideColorSpaceType)
	: Cleared(false)
	, CurrentClearColor(InClearColor)
	, InitialTextureGuid(InTextureGuid)
	, Owner(InOwner)
	, OwnerDim(InOwnerDim)
	, OwnerSize(InOwnerSize)
	, bEnableGenMips(InEnableGenMips)
	, CurrentNumMips(InEnableGenMips ? InNumMips : 1)
	, CurrentSamplerFilter(ESamplerFilter_Num)
	, CurrentMipMapBias(-1)
	, PriorSamples(FPriorSamples::Create())
{
	bUsesImageExternal =
#if PLATFORM_ANDROID
		!FAndroidMisc::ShouldUseVulkan() &&
#endif
		!Owner.NewStyleOutput && GSupportsImageExternal;
	
	if (OverrideColorSpaceType != UE::Color::EColorSpace::None)
	{
		OverrideColorSpace.Reset(new UE::Color::FColorSpace(OverrideColorSpaceType));
		check(OverrideColorSpace.IsValid());
	}
}


void FMediaTextureResource::FlushPendingData()
{
	FScopeLock Lock(&PriorSamplesCS);
	PriorSamples = FPriorSamples::Create();
}

/* FMediaTextureResource interface
 *****************************************************************************/

void FMediaTextureResource::Render(const FRenderParams& Params)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MediaTextureResource, "MediaTextureResource");

	LLM_SCOPE(ELLMTag::MediaStreaming);
	SCOPE_CYCLE_COUNTER(STAT_MediaAssets_MediaTextureResourceRender);
	CSV_SCOPED_TIMING_STAT(MediaStreaming, FMediaTextureResource_Render);

	TSharedPtr<FPriorSamples, ESPMode::ThreadSafe> LocalPriorSamples;
	{
		FScopeLock Lock(&PriorSamplesCS);
		LocalPriorSamples = PriorSamples;
	}

	LocalPriorSamples->Update();

	FLinearColor Rotation(1, 0, 0, 1);
	FLinearColor Offset(0, 0, 0, 0);

	TSharedPtr<FMediaTextureSampleSource, ESPMode::ThreadSafe> SampleSource = Params.SampleSource.Pin();

	// Do we either have a classic sample source (queue) or a single, explicit sample to display?
	if (SampleSource.IsValid() || Params.TextureSample.IsValid())
	{
		TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;
		bool UseSample;

		// Yes, it it a queue?
		if (SampleSource.IsValid())
		{
			// Yes, find out what we will display...
			UseSample = false;

			// get the most current sample to be rendered
			TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> TestSample;
			while (SampleSource->Peek(TestSample) && TestSample.IsValid())
			{
				const FMediaTimeStamp& StartTime = TestSample->GetTime();
				const FMediaTimeStamp EndTime = StartTime + TestSample->GetDuration();

				if ((Params.Rate >= 0.0f) && (Params.Time < StartTime))
				{
					break; // future sample (forward play)
				}

				if ((Params.Rate <= 0.0f) && (Params.Time >= EndTime))
				{
					break; // future sample (reverse play)
				}

#if UE_MEDIAUTILS_DEVELOPMENT_DELEGATE
				if (UseSample && Sample.IsValid())
				{
					FMediaDelegates::OnSampleDiscarded_RenderThread.Broadcast(&Owner, Sample);
				}
#endif
				UseSample = SampleSource->Dequeue(Sample);
			}
		}
		else
		{
			// We do have an explicit sample to display...
			// (or nothing)
			Sample = Params.TextureSample;
			UseSample = Sample.IsValid();
		}

#if UE_MEDIAUTILS_DEVELOPMENT_DELEGATE
		FMediaDelegates::OnPreSampleRender_RenderThread.Broadcast(&Owner, UseSample, Sample);
#endif

		// If real "external texture" support is in place and no mips are used the image will "bypass" any of this processing via the GUID-based lookup for "ExternalTextures" and will
		// reach the reading material shader without any processing here...
		// (note that if "new style output" is enabled we will ALWAYS see bUsesImageExternal as FALSE)
		if (UseSample && !(bUsesImageExternal && !bEnableGenMips))
		{
			//
			// Valid sample & Sample should be shown
			//

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
			// Placed inside the UseSample branch to guarantee the capture of a valid frame sample update
			RenderCaptureInterface::FScopedCapture RenderCapture(GMediaCaptureTexture > 0, &RHICmdList, TEXT("FMediaTextureResource::Render"));
			GMediaCaptureTexture = 0;
#endif

			const uint8 TargetNumMips = UE::MediaTexture::Private::CalcGeneratedNumMips(*Sample, bEnableGenMips);

			bool ConvertOrCopyNeeded = false;

			if (Sample->GetOutputDim().GetMin() <= 0)
			{
				//
				// Sample dimensions are invalid
				//
				ClearTexture(RHICmdList, FLinearColor::Red, false); // mark corrupt sample
			}
			else if (IMediaTextureSampleConverter *Converter = Sample->GetMediaTextureSampleConverter())
			{
				//
				// Sample brings its own converter
				//

				const uint8 SampleNumMips = Sample->GetNumMips();

				IMediaTextureSampleConverter::FConversionHints Hints;
				Hints.NumMips = (SampleNumMips > 1) ? SampleNumMips : TargetNumMips;

				const bool bNeedsUAVTexture = MediaTextureResourceHelpers::RequiresUAVTexture(Sample, Hints.NumMips);

				// Does the conversion create its own output texture?
				if ((Converter->GetConverterInfoFlags() & IMediaTextureSampleConverter::ConverterInfoFlags_WillCreateOutputTexture) == 0)
				{
					// No. Does it actually do the conversion or just a pre-process step not yielding real output?
					if (Converter->GetConverterInfoFlags() & IMediaTextureSampleConverter::ConverterInfoFlags_PreprocessOnly)
					{
						// Preprocess...
						FTextureRHIRef DummyTexture;
						if (Converter->Convert(RHICmdList, DummyTexture, Hints))
						{
							// ...followed by the built in conversion code as needed...
							ConvertOrCopyNeeded = true;
						}
					}
					else
					{
						// Conversion is fully handled by converter

						CreateIntermediateRenderTarget(RHICmdList, Sample->GetOutputDim(), MediaTextureResourceHelpers::GetConvertedPixelFormat(Sample), MediaTextureResourceHelpers::RequiresSrgbTexture(Sample), Params.ClearColor, Hints.NumMips, bNeedsUAVTexture, Sample->GetIntermediateTargetCreateFlagsHint());
						Converter->Convert(RHICmdList, RenderTargetTextureRHI, Hints);
					}
				}
				else
				{
					// The converter will create its own output texture for us to use
					FTextureRHIRef OutTexture;
					if (Converter->Convert(RHICmdList, OutTexture, Hints))
					{
						// As the converter created the texture, we might need to convert it even more to make it fit our needs. Check...
						if (RequiresConversion(OutTexture, Sample->GetOutputDim(), TargetNumMips))
						{
							CreateIntermediateRenderTarget(RHICmdList, Sample->GetOutputDim(), MediaTextureResourceHelpers::GetConvertedPixelFormat(Sample), MediaTextureResourceHelpers::RequiresSrgbTexture(Sample), Params.ClearColor, Hints.NumMips, bNeedsUAVTexture, Sample->GetIntermediateTargetCreateFlagsHint());
							ConvertTextureToOutput(RHICmdList, OutTexture.GetReference(), Sample);
						}
						else
						{
							UpdateTextureReference(RHICmdList, OutTexture);
							if (Converter->GetConverterInfoFlags() & IMediaTextureSampleConverter::ConverterInfoFlags_PreprocessOnly)
							{
								// ...followed by the built in conversion code as needed...
								ConvertOrCopyNeeded = true;
							}
						}
					}
				}

				Cleared = false;
			}
			else
			{
				// No custom conversion, we need default processing...

				// Sanity check: do we have any data?
				if (!Sample->IsExternalImage() && Sample->GetTexture() == nullptr && Sample->GetBuffer() == nullptr)
				{
					UseSample = false;
				}
				else
				{
					ConvertOrCopyNeeded = true;
				}
			}

			if (UseSample)
			{
				if (ConvertOrCopyNeeded)
				{
					if (RequiresConversion(Sample, TargetNumMips))
					{
						//
						// Sample needs to be converted by built in converter code
						//
						ConvertSample(RHICmdList, Sample, Params.ClearColor, TargetNumMips);
					}
					else
					{
						//
						// Sample can be used directly or is a simple copy
						//
						CopySample(RHICmdList, Sample, Params.ClearColor, TargetNumMips, Params.CurrentGuid);
					}
				}

				if (IMediaTextureSampleColorConverter* Converter = Sample->GetMediaTextureSampleColorConverter())
				{
					FTextureRHIRef TextureRef = IntermediateTarget ? IntermediateTarget : RenderTargetTextureRHI;

					if (bRecreateOutputTarget && TextureRef)
					{
						const static FLazyName ClassName(TEXT("FMediaTextureResource"));

						const FRHITextureDesc& IntermediateTextureDesc = TextureRef->GetDesc();

						const FRHITextureCreateDesc Desc =
							FRHITextureCreateDesc::Create2D(TEXT("MediaTextureColorConversionOutput"),
								IntermediateTextureDesc.GetSize().X,
								IntermediateTextureDesc.GetSize().Y,
								IntermediateTextureDesc.Format)
							.SetNumMips(IntermediateTextureDesc.NumMips)
							.SetClearValue(IntermediateTextureDesc.ClearValue)
							.SetClassName(ClassName)
							.SetOwnerName(GetOwnerName())
							.SetFlags(IntermediateTextureDesc.Flags);

						OutputTarget = RHICmdList.CreateTexture(Desc);
						OutputTarget->SetName(TEXT("MediaTextureColorConversionOutput"));
						OutputTarget->SetOwnerName(GetOwnerName());

						bRecreateOutputTarget = false;
					}

					Converter->ApplyColorConversion(RHICmdList, TextureRef, OutputTarget);

					if (RenderTargetTextureRHI != OutputTarget)
					{
						UpdateTextureReference(RHICmdList, OutputTarget);
					}
				}
				else if (IntermediateTarget)
				{
					OutputTarget = IntermediateTarget;
				}

				Rotation = Sample->GetScaleRotation();
				Offset = Sample->GetOffset();

				if (CurrentSample)
				{
					// If we had a current sample (directly used as output), we can now schedule its retirement
					LocalPriorSamples->Retire(RHICmdList, CurrentSample);
					CurrentSample = nullptr;
				}

				// Do we use a local copy as our output?
				if (OutputTarget == RenderTargetTextureRHI)
				{
					// Yes, we can schedule the actual sample for retirement right away
					LocalPriorSamples->Retire(RHICmdList, Sample);
				}
				else
				{
					// No, we need to hold on to the sample
					CurrentSample = Sample;
				}

				// Generate mips as needed
				if (CurrentNumMips > 1 && !Cleared && Sample->GetNumMips() == 1 && !IsBlockCompressedFormat(OutputTarget->GetFormat()))
				{
					check(OutputTarget);

					CacheRenderTarget(OutputTarget, TEXT("MipGeneration"), MipGenerationCache);

					FRDGBuilder GraphBuilder(RHICmdList);
					FRDGTextureRef MipOutputTexture = GraphBuilder.RegisterExternalTexture(MipGenerationCache);
					FGenerateMips::Execute(GraphBuilder, GetFeatureLevel(), MipOutputTexture, FGenerateMipsParams{ SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp });
					GraphBuilder.Execute();
				}

				SET_FLOAT_STAT(STAT_MediaUtils_TextureSampleTime, Sample->GetTime().Time.GetTotalMilliseconds());

				// Notify extensions the media texture has been updated with a new sample.
				FMediaDelegates::OnSampleRendered_RenderThread.Broadcast(this, Sample);
			}
		}
	}
	else if (Params.CanClear)
	{
		//
		// No valid sample source & we should clear
		//

		// Need to clear the output?
		if (!Cleared || (Params.ClearColor != CurrentClearColor))
		{
			// Yes...
			ClearTexture(RHICmdList, Params.ClearColor, false);

			if (CurrentSample)
			{
				// If we had a current sample (directly used as output), we can now schedule its retirement
				LocalPriorSamples->Retire(RHICmdList, CurrentSample);
				CurrentSample = nullptr;
			}
		}
	}

	// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -
	// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -

	// Cache next available sample time in the MediaTexture owner since we're the only one that can consume from the queue
	CacheNextAvailableSampleTime(SampleSource);

	// Update external texture registration in case we have no native support
	// (in that case there is support, the player will do this - but it is used all the time)
	if (!Owner.NewStyleOutput && !bUsesImageExternal)
	{
		SetupSampler();

		if (Params.CurrentGuid.IsValid())
		{
			FTextureRHIRef VideoTexture = (FTextureRHIRef)Owner.TextureReference.TextureReferenceRHI;
			FExternalTextureRegistry::Get().RegisterExternalTexture(Params.CurrentGuid, VideoTexture, SamplerStateRHI, Rotation, Offset);
		}

		if (Params.PreviousGuid.IsValid() && (Params.PreviousGuid != Params.CurrentGuid))
		{
			FExternalTextureRegistry::Get().UnregisterExternalTexture(Params.PreviousGuid);
		}
	}
	
	// Update usable Guid for the RenderThread
	Owner.SetRenderedExternalTextureGuid(Params.CurrentGuid);
}


/* FRenderTarget interface
 *****************************************************************************/

FIntPoint FMediaTextureResource::GetSizeXY() const
{
	return FIntPoint(Owner.GetWidth(), Owner.GetHeight());
}


/* FTextureResource interface
 *****************************************************************************/

FString FMediaTextureResource::GetFriendlyName() const
{
	return Owner.GetPathName();
}


uint32 FMediaTextureResource::GetSizeX() const
{
	return Owner.GetWidth();
}


uint32 FMediaTextureResource::GetSizeY() const
{
	return Owner.GetHeight();
}


void FMediaTextureResource::SetupSampler()
{
	ESamplerFilter OwnerFilter = (CurrentNumMips > 1 || Owner.NewStyleOutput) ? (ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(&Owner) : SF_Bilinear;
	float OwnerMipMapBias = Owner.GetMipMapBias();

	if (CurrentSamplerFilter != OwnerFilter || !FMath::IsNearlyEqual(CurrentMipMapBias, OwnerMipMapBias))
	{
		CurrentSamplerFilter = OwnerFilter;
		CurrentMipMapBias = OwnerMipMapBias;

		// create the sampler state
		FSamplerStateInitializerRHI SamplerStateInitializer(
			CurrentSamplerFilter,
			(Owner.AddressX == TA_Wrap) ? AM_Wrap : ((Owner.AddressX == TA_Clamp) ? AM_Clamp : AM_Mirror),
			(Owner.AddressY == TA_Wrap) ? AM_Wrap : ((Owner.AddressY == TA_Clamp) ? AM_Clamp : AM_Mirror),
			AM_Wrap,
			CurrentMipMapBias
		);

		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}
}


void FMediaTextureResource::InitRHI(FRHICommandListBase& RHICmdListBase)
{
	FRHICommandListImmediate& RHICmdList = RHICmdListBase.GetAsImmediate();

	SetupSampler();
	
	// Note: set up default texture, or we can get sampler bind errors on render
	// we can't leave here without having a valid bindable resource for some RHIs.

	ClearTexture(RHICmdList, CurrentClearColor, Owner.SRGB);

	// Make sure init has done it's job - we can't leave here without valid bindable resources for some RHI's
	check(TextureRHI.IsValid());
	check(RenderTargetTextureRHI.IsValid());
	check(IntermediateTarget.IsValid());

	// Register "external texture" parameters if the platform does not support them (and hence the player does not set them)
	if (!bUsesImageExternal)
	{
		FTextureRHIRef VideoTexture = (FTextureRHIRef)Owner.TextureReference.TextureReferenceRHI;
		FExternalTextureRegistry::Get().RegisterExternalTexture(InitialTextureGuid, VideoTexture, SamplerStateRHI);
	}
}


void FMediaTextureResource::ReleaseRHI()
{
	if (RenderTargetTextureRHI.IsValid())
	{
		FMediaDelegates::OnMediaTextureResourceReleased_RenderThread.Broadcast(RenderTargetTextureRHI.GetReference());
	}

	Cleared = false;

	MipGenerationCache.SafeRelease();

	InputTarget.SafeRelease();
	IntermediateTarget.SafeRelease();
	OutputTarget.SafeRelease();
	RenderTargetTextureRHI.SafeRelease();
	TextureRHI.SafeRelease();

	UpdateTextureReference(FRHICommandListImmediate::Get(), nullptr);
}


/* FMediaTextureResource implementation
 *****************************************************************************/

void FMediaTextureResource::SetJustInTimeRenderParams(const FRenderParams& InJustInTimeRenderParams)
{
	check(IsInRenderingThread());

	JustInTimeRenderParams = MakeUnique<FRenderParams>();
	*JustInTimeRenderParams = InJustInTimeRenderParams;
}

void FMediaTextureResource::ResetJustInTimeRenderParams()
{
	check(IsInRenderingThread());

	JustInTimeRenderParams.Reset();
}

void FMediaTextureResource::JustInTimeRender()
{
	check(IsInRenderingThread());

	if (JustInTimeRenderParams.IsValid())
	{
		Render(*JustInTimeRenderParams);
		JustInTimeRenderParams.Reset();
	}
}


void FMediaTextureResource::ClearTexture(FRHICommandListImmediate& RHICmdList, const FLinearColor& ClearColor, bool SrgbOutput)
{
	// create output render target if we don't have one yet
	constexpr uint8 NumMips = 1;
	constexpr bool bNeedsUAVTexture = false;
	CreateIntermediateRenderTarget(RHICmdList, FIntPoint(2, 2), PF_B8G8R8A8, SrgbOutput, ClearColor, NumMips, bNeedsUAVTexture);

	// draw the clear color
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MediaTextureResource, "FMediaTextureResource_ClearTexture");

		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::SRVMask, ERHIAccess::RTV));

		FRHIRenderPassInfo RPInfo(RenderTargetTextureRHI, ERenderTargetActions::Clear_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearTexture"));
		RHICmdList.EndRenderPass();
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}

	Cleared = true;
}


bool FMediaTextureResource::RequiresConversion(const FTextureRHIRef& SampleTexture, const FIntPoint & OutputDim, uint8 InTargetNumMips) const
{
	if (Owner.NewStyleOutput)
	{
		//
		// New Style
		// 

		// If we have no mips in the sample, but want to have some in the output, we use the conversion pass
		// to setup level 0 and have a suitable output texture
		if (SampleTexture->GetNumMips() == 1 && InTargetNumMips != 1)
		{
			return true;
		}
	}

	if (SampleTexture->GetSizeXY() != OutputDim)
	{
		return true;
	}

	// Only the following pixel formats are supported natively.
	// All other formats require a conversion on the GPU.

	const EPixelFormat Format = SampleTexture->GetFormat();

	return ((Format != PF_B8G8R8A8) &&
		(Format != PF_R8G8B8A8) &&
		(Format != PF_FloatRGB) &&
		(Format != PF_A16B16G16R16) &&
		(Format != PF_FloatRGBA));
}
 

bool FMediaTextureResource::RequiresConversion(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample, uint8 InTargetNumMips) const
{
	if (Owner.NewStyleOutput)
	{
		//
		// New Style
		// 

		FRHITexture *Texture = Sample->GetTexture();

		// If we have no mips in the sample, but want to have some in the output, we use the conversion pass
		// to setup level 0 and have a suitable output texture
		if (Texture && Texture->GetNumMips() == 1 && InTargetNumMips != 1)
		{
			return true;
		}
	}

	// If the output dimensions are not the same as the sample's
	// dimensions, a resizing conversion on the GPU is required.
	//
	// Some reasons this could trigger:
	//	- Cropping
	//  - NV12's 1.5x height factor -> but the format would need conversion to RGB anyway
	//	- Various YCbCr formats will be stored with different widths as the final RGB output data -> a conversion is needed anyway
	//
	if (Sample->GetDim() != Sample->GetOutputDim())
	{
		return true;
	}

	// Any reason color conversion must run?
	const UE::Color::FColorSpace& DestinationCS = OverrideColorSpace.IsValid() ? *OverrideColorSpace : UE::Color::FColorSpace::GetWorking();
	if (Sample->ShouldApplyColorConversion())
	{
		// Color space mismatch with the working color space.
		if (!Sample->GetSourceColorSpace().Equals(DestinationCS))
		{
			return true;
		}

		// Non-unity HDR nits normalization is applied as a scale on the CSTransform inside
		// GetColorSpaceConversionMatrixForSample, so the conversion pass must run even when
		// the color space and encoding would otherwise allow us to skip it.
		const float NormalizationFactor = Sample->GetHDRNitsNormalizationFactor();
		if (!FMath::IsNearlyEqual(NormalizationFactor, 1.0f))
		{
			return true;
		}
	}

	// No. Lets check if the encoding type is linear or not...
	UE::Color::EEncoding ColorEncoding = Sample->GetEncodingType();
	EMediaTextureSampleFormat Format = Sample->GetFormat();

	bool b8BitRGBA = (Format == EMediaTextureSampleFormat::CharBGRA ||
					  Format == EMediaTextureSampleFormat::CharRGBA ||
					  Format == EMediaTextureSampleFormat::DXT1 ||
					  Format == EMediaTextureSampleFormat::DXT5);

	if (b8BitRGBA)
	{
		// sRGB?
		if (ColorEncoding == UE::Color::EEncoding::sRGB)
		{
			// 8-bit RGBA and either CPU side buffer or texture with sRGB flag -> we can assume an inpout texture with HW translation to linear
			auto Texture = Sample->GetTexture();
			if (Texture == nullptr || ((uint32)Texture->GetFlags() & (uint32)TexCreate_SRGB))
			{
				return false;
			}
		}
	}

	bool bRGBA = b8BitRGBA ||
				 Format == EMediaTextureSampleFormat::FloatRGB ||
				 Format == EMediaTextureSampleFormat::FloatRGBA ||
				 Format == EMediaTextureSampleFormat::ABGR16 ||
				 Format == EMediaTextureSampleFormat::RGBA16 ||
				 Format == EMediaTextureSampleFormat::CharBGR10A2;

	// RGBA and linear?
	if ((bRGBA && ColorEncoding == UE::Color::EEncoding::Linear ) || !Sample->ShouldApplyColorConversion())
	{
		return false;
	}

	// Either we have RGBA with non-linear encoding or its not RGBA -> we must convert!
	return true;
}



void FMediaTextureResource::ConvertSample(FRHICommandListImmediate& RHICmdList, const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample, const FLinearColor& ClearColor, uint8 InTargetNumMips)
{
	const EPixelFormat InputPixelFormat = MediaTextureResourceHelpers::GetPixelFormat(Sample);

	const uint8 SampleNumMips = Sample->GetNumMips();

	// get input texture
	FRHITexture* InputTexture = nullptr;
	{
		// If the sample already provides a texture resource, we simply use that
		// as the input texture. If the sample only provides raw data, then we
		// create our own input render target and copy the data into it.

		FRHITexture* SampleTexture = Sample->GetTexture();

		if (SampleTexture)
		{
			// Use the sample as source texture...

			InputTexture = SampleTexture;
			UpdateResourceSize();
			InputTarget = nullptr;
		}
		else
		{
			// Make a source texture so we can convert from it...

			const bool SrgbTexture = MediaTextureResourceHelpers::RequiresSrgbInputTexture(Sample);
			const ETextureCreateFlags InputCreateFlags = SrgbTexture ? TexCreate_SRGB : TexCreate_None;
			const FIntPoint SampleDim = Sample->GetDim();

			// create a new temp input render target if necessary
			if (!InputTarget.IsValid() || (InputTarget->GetSizeXY() != SampleDim) || (InputTarget->GetFormat() != InputPixelFormat) || ((InputTarget->GetFlags() & InputCreateFlags) != InputCreateFlags) || (InputTarget->GetNumMips() != SampleNumMips))
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("FMediaTextureResource"), SampleDim, InputPixelFormat)
					.SetNumMips(SampleNumMips)
					.SetFlags(InputCreateFlags);

				InputTarget = RHICmdList.CreateTexture(Desc);

				UpdateResourceSize();
			}

			// copy sample data to input render target
			MediaTextureResourceHelpers::UpdateTextureFromSample(RHICmdList, Sample, SampleNumMips, InputTarget);

			InputTexture = InputTarget;
		}
	}

	// create the output texture
	const FIntPoint OutputDim = Sample->GetOutputDim();
	const uint8 NumMips = (SampleNumMips > 1) ? SampleNumMips : InTargetNumMips;
	const bool bNeedsUAVTexture = MediaTextureResourceHelpers::RequiresUAVTexture(Sample, NumMips);
	CreateIntermediateRenderTarget(RHICmdList, OutputDim, MediaTextureResourceHelpers::GetConvertedPixelFormat(Sample), MediaTextureResourceHelpers::RequiresSrgbTexture(Sample), ClearColor, NumMips, bNeedsUAVTexture, Sample->GetIntermediateTargetCreateFlagsHint());

	ConvertTextureToOutput(RHICmdList, InputTexture, Sample);
}


void FMediaTextureResource::GetColorSpaceConversionMatrixForSample(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample, FMatrix44f& ColorSpaceMtx)
{
	const UE::Color::FColorSpace& DestinationCS = OverrideColorSpace.IsValid() ? *OverrideColorSpace : UE::Color::FColorSpace::GetWorking();
	
	if (Sample->GetMediaTextureSampleColorConverter() || Sample->GetSourceColorSpace().Equals(DestinationCS))
	{
		ColorSpaceMtx = FMatrix44f::Identity;
	}
	else
	{
		// Apply the color space transformation from source to destination.
		ColorSpaceMtx = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(Sample->GetSourceColorSpace(), DestinationCS, Sample->GetChromaticAdapationMethod()));
	}
	
	const float NormalizationFactor = Sample->GetHDRNitsNormalizationFactor();
	if (!FMath::IsNearlyEqual(NormalizationFactor, 1.0f))
	{
		ColorSpaceMtx = ColorSpaceMtx.ApplyScale(NormalizationFactor);
	}
}


 void FMediaTextureResource::ConvertTextureToOutput(FRHICommandListImmediate& RHICmdList, FRHITexture* InputTexture, const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
 {
	// perform the conversion
	{
		// We should never get here with a sample that contains mips!
		check(Sample->GetNumMips() == 1);

		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MediaTextureResource, "FMediaTextureResource_Convert");

		// draw full size quad into render target
		// This needs to happen before we begin to setup the draw call, because on DX11, this might flush the command list more or less randomly
		FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		FRHITexture* RenderTarget = IntermediateTarget;
		RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::Unknown, ERHIAccess::RTV));

		FIntPoint OutputDim(RenderTarget->GetSizeXYZ().X, RenderTarget->GetSizeXYZ().Y);

		// note: we are not explicitly transitioning the input texture to be readable here
		// (we assume this to be the case already - main as some platforms may fail to orderly transition the resource due to special cases regarding their internal setup)
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));

		FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::DontLoad_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia"));
		{
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

			// configure media shaders
			auto ShaderMap = GetGlobalShaderMap(GetFeatureLevel());
			TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

			// Temporary SRV variables to hold references for the draw
			FShaderResourceViewRHIRef TempSRV0, TempSRV1;

			// Use the sample format to choose the conversion path
			EMediaTextureSampleFormat SampleFormat = Sample->GetFormat();
			switch (SampleFormat)
			{
				// Planar 8-bit YCbCr 420
				case EMediaTextureSampleFormat::CharNV12:
				case EMediaTextureSampleFormat::CharNV21:	// swapped CbCr (vs. NV12)
				{
					auto YUVMtx = Sample->GetSampleToRGBMatrix();
					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);
					
					if (InputTexture->GetFormat() == PF_NV12)
					{
						TShaderMapRef<FNV12ConvertPS> ConvertShader(ShaderMap);
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
						FIntPoint TexDim = InputTexture->GetSizeXY();
						TempSRV0 = RHICmdList.CreateShaderResourceView(
							InputTexture,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(InputTexture)
								.SetFormat(PF_G8)); // note: RHI does provide "magic" to select Y vs. UV planes based on the pixel format (D3D/DXGI)
						TempSRV1 = RHICmdList.CreateShaderResourceView(
							InputTexture,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(InputTexture)
								.SetFormat(PF_R8G8));
						SetShaderParametersLegacyPS(RHICmdList, ConvertShader, TexDim, TempSRV0, TempSRV1, OutputDim, YUVMtx, Sample->GetEncodingType(), ColorSpaceMtx, SampleFormat == EMediaTextureSampleFormat::CharNV21, Sample->GetToneMapMethod());
					}
					else
					{
						TShaderMapRef<FNV12ConvertAsBytesPS> ConvertShader(ShaderMap);
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
						SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, YUVMtx, Sample->GetEncodingType(), ColorSpaceMtx, SampleFormat == EMediaTextureSampleFormat::CharNV21, Sample->GetToneMapMethod());
					}
				}
				break;

				// Planar 10-bit YCbCr 420 (in 16-bit components)
				case EMediaTextureSampleFormat::P010:
				{
					auto YUVMtx = Sample->GetSampleToRGBMatrix();
					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					FIntPoint TexDim = InputTexture->GetSizeXY();
					if (InputTexture->GetFormat() == PF_P010)
					{
						TShaderMapRef<FP010ConvertPS> ConvertShader(ShaderMap);
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						FShaderResourceViewRHIRef Y_SRV = RHICmdList.CreateShaderResourceView(
							InputTexture,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(InputTexture)
								.SetFormat(PF_G16)); // note: RHI does provide "magic" to select Y vs. UV planes based on the pixel format (D3D/DXGI)
						FShaderResourceViewRHIRef UV_SRV = RHICmdList.CreateShaderResourceView(
							InputTexture,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(InputTexture)
								.SetFormat(PF_G16R16));
						SetShaderParametersLegacyPS(RHICmdList, ConvertShader, TexDim, Y_SRV, UV_SRV, OutputDim, YUVMtx, ColorSpaceMtx, Sample->GetEncodingType(), Sample->GetToneMapMethod());
					}
					else
					{
						TShaderMapRef<FP010ConvertAsUINT16sPS> ConvertShader(ShaderMap);
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						SetShaderParametersLegacyPS(RHICmdList, ConvertShader, TexDim, InputTexture, OutputDim, YUVMtx, ColorSpaceMtx, Sample->GetEncodingType(), Sample->GetToneMapMethod());
					}
				}
				break;
 
				// Various YCbCr 422 formats
				case EMediaTextureSampleFormat::CharYUY2:		// Y0CbY1Cr
				case EMediaTextureSampleFormat::CharYVYU:		// Y0CrY1Cb
				case EMediaTextureSampleFormat::Char2VUY:		// CbY0CrY1
				case EMediaTextureSampleFormat::YUVv216:		// CbY0CrY1
				case EMediaTextureSampleFormat::ShortYUY2:		// Y0CbY1Cr
				case EMediaTextureSampleFormat::CharUYVY:		// CbY0CrY1
				{
					auto YUVMtx = Sample->GetSampleToRGBMatrix();
					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					TShaderMapRef<FYUVv216ConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, YUVMtx, Sample->GetEncodingType(), ColorSpaceMtx,
												SampleFormat!= EMediaTextureSampleFormat::CharYUY2 && SampleFormat != EMediaTextureSampleFormat::CharYVYU && SampleFormat != EMediaTextureSampleFormat::ShortYUY2,	// Y or Cb first
												InputTexture->GetFormat() == PF_B8G8R8A8,																	// ARGB vs. ABGR (memory order)
												SampleFormat == EMediaTextureSampleFormat::CharYVYU,														// Cb / Cr swap
												Sample->GetToneMapMethod(),
												SampleFormat == EMediaTextureSampleFormat::CharUYVY);  // Whether to use bilinear sampling for the chroma values.
				}
				break;

				// 10-bit YCbCr 422 format (stored in A2R10G10B10)
				case EMediaTextureSampleFormat::YUVv210:
				{
					auto YUVMtx = Sample->GetSampleToRGBMatrix();
					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					TShaderMapRef<FYUVv210ConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, YUVMtx, Sample->GetEncodingType(), ColorSpaceMtx,
												true, Sample->GetToneMapMethod());
				}
				break;

				// Various YCbCR 444 formats
				case EMediaTextureSampleFormat::CharAYUV:
				case EMediaTextureSampleFormat::Y416:
				case EMediaTextureSampleFormat::R4FL:
				{
					auto YUVMtx = Sample->GetSampleToRGBMatrix();
					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					TShaderMapRef<FYUVY416ConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(
						InputTexture,
						FRHIViewDesc::CreateTextureSRV()
							.SetDimensionFromTexture(InputTexture)
							.SetFormat((Sample->GetFormat() == EMediaTextureSampleFormat::Y416) ? PF_A16B16G16R16 : PF_A32B32G32R32F));

					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, SRV, YUVMtx, Sample->GetEncodingType(), ColorSpaceMtx,
												InputTexture->GetFormat() == PF_A8R8G8B8, Sample->GetToneMapMethod());
				}
				break;

				// Various RGB(A) formats
				case EMediaTextureSampleFormat::CharBGR10A2:
				case EMediaTextureSampleFormat::CharRGBA:
				case EMediaTextureSampleFormat::CharBGRA:
				case EMediaTextureSampleFormat::FloatRGB:
				case EMediaTextureSampleFormat::FloatRGBA:
				case EMediaTextureSampleFormat::RGBA16:
				case EMediaTextureSampleFormat::ABGR16:
				{
					// We get here if we detect the need for an inverse EOTF application, CS conversion OR if we need to generate mips for RGBA samples

					// Inverse EOTF?
					UE::Color::EEncoding Encoding;
					if ((uint32)InputTexture->GetFlags() & (uint32)TexCreate_SRGB)
					{
						// We do not need one: sRGB will be handled in HW
						Encoding = UE::Color::EEncoding::Linear;
					}
					else
					{
						Encoding = Sample->GetEncodingType();
					}

					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					TShaderMapRef<FRGBConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, Encoding, ColorSpaceMtx, Sample->GetToneMapMethod());
				}
				break;

				case EMediaTextureSampleFormat::ARGB16_BIG:
				{
					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					TShaderMapRef<FARGB16BigConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(
						InputTexture,
						FRHIViewDesc::CreateTextureSRV()
							.SetDimensionFromTexture(InputTexture)
							.SetFormat(PF_R16G16B16A16_UINT));

					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, SRV, OutputDim, Sample->GetEncodingType(), ColorSpaceMtx, Sample->GetToneMapMethod());
				}
				break;

				case EMediaTextureSampleFormat::CharBMP:
				{
					// Simple 1:1 copy plus flip & color adjustment (but using normal texture sampler: sRGB conversions may occur depending on setup; any manual sRGB/linear conversion is disabled)
					TShaderMapRef<FBMPConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, false);
				}
				break;

				case EMediaTextureSampleFormat::DXT1:
				case EMediaTextureSampleFormat::DXT5:
				{
					// Note: the shader used here will pass-through alpha data
					
					// Inverse EOTF?
					UE::Color::EEncoding Encoding;
					if ((uint32)InputTexture->GetFlags() & (uint32)TexCreate_SRGB)
					{
						// We do not need one: sRGB will be handled in HW
						Encoding = UE::Color::EEncoding::Linear;
					}
					else
					{
						Encoding = Sample->GetEncodingType();
					}

					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					// Convert the DXn compressed data into RGBA so we can more easily process it using the existing logic (e.g mip generation)
					// (this also will deal with any sRGB/Rec703 conversions - as well as any color space conversions)
					TShaderMapRef<FRGBConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, Encoding, ColorSpaceMtx, Sample->GetToneMapMethod());
				}
				break;

				case EMediaTextureSampleFormat::BC4:
				{
					// We are using the RGB conversion shader to just pipe BC4 data (read into R) and remap it to be usable with L8 or A8 etc. (RRRR) using the otherwise idle CS conversion matrix
					// (note: no EOTF or CS - this is alpha only!)
					TShaderMapRef<FRGBConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, UE::Color::EEncoding::Linear, SplatMtx);
				}
				break;

				case EMediaTextureSampleFormat::YCoCg_DXT5:
				{
					// Inverse EOTF?
					UE::Color::EEncoding Encoding;
					if ((uint32)InputTexture->GetFlags() & (uint32)TexCreate_SRGB)
					{
						// We do not need one: sRGB will be handled in HW
						Encoding = UE::Color::EEncoding::Linear;
					}
					else
					{
						Encoding = Sample->GetEncodingType();
					}

					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					// Convert the DXn compressed YCoCg data into RGBA so we can more easily process it using the existing logic (e.g mip generation)
					// (this also will deal with any sRGB/Rec703 conversions - as well as any color space conversions)
					TShaderMapRef<FYCoCgConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, Encoding, ColorSpaceMtx, Sample->GetToneMapMethod());
				}
				break;

				case EMediaTextureSampleFormat::ExternalVYU:
				{
					auto YUVMtx = Sample->GetSampleToRGBMatrix();
					FMatrix44f ColorSpaceMtx;
					GetColorSpaceConversionMatrixForSample(Sample, ColorSpaceMtx);

					TShaderMapRef<FVYUConvertPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTexture, OutputDim, YUVMtx, Sample->GetEncodingType(), ColorSpaceMtx, Sample->GetToneMapMethod());
				}
				break;

				default:
				{
					// This should not happen in normal use: still - end the render pass to avoid any trouble with RHI
					RHICmdList.EndRenderPass();
					return; // unsupported format (either illegal value or needing a custom converter)
				}
			}

			RHICmdList.SetStreamSource(0, VertexBuffer, 0);
			// set viewport to RT size
			RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

			RHICmdList.DrawPrimitive(0, 2, 1);
		}
		RHICmdList.EndRenderPass();
		RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::RTV, ERHIAccess::SRVGraphics));
	}

	Cleared = false;
}


void FMediaTextureResource::CopySample(FRHICommandListImmediate& RHICmdList, const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample, const FLinearColor& ClearColor, uint8 InTargetNumMips, const FGuid & TextureGUID)
{
	FRHITexture* SampleTexture = Sample->GetTexture();
	const uint8 SampleNumMips = Sample->GetNumMips();

	// If the sample already provides a texture resource, we simply use that
	// as the output render target. If the sample only provides raw data, then
	// we create our own output render target and copy the data into it.

	if (SampleTexture != nullptr)
	{
		// Use sample's texture as the new render target - no copy
		if (TextureRHI != SampleTexture)
		{
			UpdateTextureReference(RHICmdList, SampleTexture);

			MipGenerationCache.SafeRelease();
			IntermediateTarget.SafeRelease();
		}
		else
		{
			// Texture to receive texture from sample
			const uint8 NumMips = (SampleNumMips > 1) ? SampleNumMips : InTargetNumMips;
			const bool bNeedsUAVTexture = MediaTextureResourceHelpers::RequiresUAVTexture(Sample, NumMips);
			CreateIntermediateRenderTarget(RHICmdList, Sample->GetOutputDim(), MediaTextureResourceHelpers::GetPixelFormat(Sample), MediaTextureResourceHelpers::RequiresSrgbTexture(Sample), ClearColor, NumMips, bNeedsUAVTexture, Sample->GetIntermediateTargetCreateFlagsHint());

			// Copy data into the output texture to able to add mips later on
			TransitionAndCopyTexture(RHICmdList, SampleTexture, IntermediateTarget, FRHICopyTextureInfo());
		}
	}
	else
	{
		// Texture to receive precisely only output pixels via CPU copy
		const uint8 NumMips = (SampleNumMips > 1) ? SampleNumMips : InTargetNumMips;
		const bool bNeedsUAVTexture = MediaTextureResourceHelpers::RequiresUAVTexture(Sample, NumMips);
		CreateIntermediateRenderTarget(RHICmdList, Sample->GetDim(), MediaTextureResourceHelpers::GetPixelFormat(Sample), MediaTextureResourceHelpers::RequiresSrgbTexture(Sample), ClearColor, NumMips, bNeedsUAVTexture, Sample->GetIntermediateTargetCreateFlagsHint());

		// If we also have no source buffer and the platform generally would allow for use of external textures, we assume it is just that...
		// (as long as the player actually produces (dummy) samples, this will enable mips support as well as auto conversion for "new style output" mode)
		if (!Sample->GetBuffer())
		{
			// we expect an external texture image in this case - we should have no mips reported here!
			check(SampleNumMips == 1);

			if (GSupportsImageExternal)
			{
				CopyFromExternalTexture(RHICmdList, Sample, TextureGUID);
			}
			else
			{
				// We never should get here, but could should a player pass us a "valid" sample with neither texture or buffer based data in it (and we don't have ExternalTexture support)

				// Just clear the texture so we don't show any random memory contents...
				RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));
				FRHIRenderPassInfo RPInfo(RenderTargetTextureRHI, ERenderTargetActions::Clear_Store);
				RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearTexture"));
				RHICmdList.EndRenderPass();
				RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));
			}
		}
		else
		{
			IMediaTextureSampleConverter::FConversionHints Hints;
			Hints.NumMips = NumMips;
			IMediaTextureSampleConverter* Converter = Sample->GetMediaTextureSampleConverter();
			if (!Converter || !Converter->Convert(RHICmdList, RenderTargetTextureRHI, Hints))
			{
				// Copy sample data (from CPU mem) to output render target
				MediaTextureResourceHelpers::UpdateTextureFromSample(RHICmdList, Sample, SampleNumMips, RenderTargetTextureRHI);
			}

			// Make sure resource is in SRV mode again
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI.GetReference(), ERHIAccess::Unknown, ERHIAccess::SRVMask));
		}
	}

	Cleared = false;
}


void FMediaTextureResource::CopyFromExternalTexture(FRHICommandListImmediate& RHICmdList, const TSharedPtr <IMediaTextureSample, ESPMode::ThreadSafe>& Sample, const FGuid & TextureGUID)
{
	{
		FTextureRHIRef SampleTexture;
		FSamplerStateRHIRef SamplerState;
		if (!FExternalTextureRegistry::Get().GetExternalTexture(nullptr, TextureGUID, SampleTexture, SamplerState))
		{
			// This should never happen: we could not find the external texture data. Still, if it does we clear the output...
			FRHIRenderPassInfo RPInfo(RenderTargetTextureRHI, ERenderTargetActions::Clear_Store);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearTexture"));
			RHICmdList.EndRenderPass();
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			return;
		}

		FLinearColor Offset, ScaleRotation;
		FExternalTextureRegistry::Get().GetExternalTextureCoordinateOffset(TextureGUID, Offset);
		FExternalTextureRegistry::Get().GetExternalTextureCoordinateScaleRotation(TextureGUID, ScaleRotation);

		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MediaTextureResource, "FMediaTextureResource_ConvertExternalTexture");

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		FRHITexture* RenderTarget = RenderTargetTextureRHI.GetReference();

		FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::DontLoad_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia_ExternalTexture"));
		{
			const FIntPoint OutputDim = Sample->GetOutputDim();

			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);
			
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

			// configure media shaders
			auto ShaderMap = GetGlobalShaderMap(GetFeatureLevel());
			TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

			TShaderMapRef<FReadTextureExternalPS> CopyShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = CopyShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			SetShaderParametersLegacyPS(RHICmdList, CopyShader, SampleTexture, SamplerState, ScaleRotation, Offset);

			// draw full size quad into render target
			FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
			RHICmdList.SetStreamSource(0, VertexBuffer, 0);
			// set viewport to RT size
			RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

			RHICmdList.DrawPrimitive(0, 2, 1);
		}
		RHICmdList.EndRenderPass();
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::SRVMask));
	}
}


void FMediaTextureResource::UpdateResourceSize()
{
	SIZE_T ResourceSize = 0;

	if (InputTarget.IsValid())
	{
		ResourceSize += CalcTextureSize(InputTarget->GetSizeX(), InputTarget->GetSizeY(), InputTarget->GetFormat(), 1);
	}
	
	if (IntermediateTarget.IsValid() && IntermediateTarget != OutputTarget)
	{
		ResourceSize += CalcTextureSize(IntermediateTarget->GetSizeX(), IntermediateTarget->GetSizeY(), IntermediateTarget->GetFormat(), 1);
	}

	if (OutputTarget.IsValid())
	{
		ResourceSize += CalcTextureSize(OutputTarget->GetSizeX(), OutputTarget->GetSizeY(), OutputTarget->GetFormat(), 1);
	}

	OwnerSize = ResourceSize;
}


void FMediaTextureResource::UpdateTextureReference(FRHICommandListImmediate& RHICmdList, FRHITexture* NewTexture)
{
	TextureRHI = NewTexture;
	RenderTargetTextureRHI = NewTexture;

	RHICmdList.UpdateTextureReference(Owner.TextureReference.TextureReferenceRHI, NewTexture);
	// note: sRGB status for Owner.SRGB is handled (on game thread) in MediaTetxure.cpp

	if (RenderTargetTextureRHI != nullptr)
	{
		OwnerDim = FIntPoint(RenderTargetTextureRHI->GetSizeX(), RenderTargetTextureRHI->GetSizeY());
	}
	else
	{
		OwnerDim = FIntPoint::ZeroValue;
	}
}

void FMediaTextureResource::CreateIntermediateRenderTarget(FRHICommandListImmediate& RHICmdList, const FIntPoint & InDim, EPixelFormat InPixelFormat, bool bInSRGB, const FLinearColor & InClearColor, uint8 InNumMips, bool bNeedsUAVSupport, ETextureCreateFlags InExtraCreateFlags)
{
	// create output render target if necessary
	ETextureCreateFlags OutputCreateFlags = bInSRGB ? TexCreate_SRGB : TexCreate_None;
	OutputCreateFlags |= InExtraCreateFlags;
	if (bNeedsUAVSupport && UE::PixelFormat::HasCapabilities(InPixelFormat, EPixelFormatCapabilities::TypedUAVLoad))
	{
		OutputCreateFlags |= TexCreate_UAV;
	}

	// Make sure we only set a number of mips that actually makes sense, given the sample size
	InNumMips = UE::MediaTexture::Private::ClampNumMips(InNumMips, InDim);

	// Avoid setting things up as a render target if we are passing in a block compression format
	// (this happens only if these have no conversion needs at all - otherwise they will be promoted to RGBA8)
	if (!IsBlockCompressedFormat(InPixelFormat))
	{
		OutputCreateFlags |= ETextureCreateFlags::RenderTargetable;
	}

	if ((InClearColor != CurrentClearColor) || !IntermediateTarget.IsValid() || (IntermediateTarget->GetSizeXY() != InDim) || (IntermediateTarget->GetFormat() != InPixelFormat) || ((IntermediateTarget->GetFlags() & OutputCreateFlags) != OutputCreateFlags) || CurrentNumMips != InNumMips)
	{
		MipGenerationCache.SafeRelease();

		const static FLazyName ClassName(TEXT("FMediaTextureResource"));
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("MediaTextureResourceOutput"))
			.SetExtent(InDim)
			.SetFormat(InPixelFormat)
			.SetNumMips(InNumMips)
			.SetFlags(OutputCreateFlags | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetClearValue(FClearValueBinding(InClearColor))
			.SetClassName(ClassName)
			.SetOwnerName(GetOwnerName());

		IntermediateTarget = RHICmdList.CreateTexture(Desc);

		IntermediateTarget->SetName(TEXT("MediaTextureResourceOutput"));
		IntermediateTarget->SetOwnerName(GetOwnerName());

		CurrentClearColor = InClearColor;
		CurrentNumMips = InNumMips;
		UpdateResourceSize();

		Cleared = false;
		bRecreateOutputTarget = true;
	}
	
	if (RenderTargetTextureRHI != IntermediateTarget)
	{
		UpdateTextureReference(RHICmdList, IntermediateTarget);
	}
}

void FMediaTextureResource::CacheNextAvailableSampleTime(const TSharedPtr<FMediaTextureSampleSource, ESPMode::ThreadSafe>& InSampleQueue) const
{
	FTimespan SampleTime(FTimespan::MinValue());

	if (InSampleQueue.IsValid())
	{
		TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;
		if (InSampleQueue->Peek(Sample))
		{
			SampleTime = Sample->GetTime().Time;
		}
	}

	Owner.CacheNextAvailableSampleTime(SampleTime);
}
