// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Queue.h"
#include "Math/Color.h"
#include "MediaSampleSource.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "TextureResource.h"
#include "UnrealClient.h"
#include "IMediaTimeSource.h"
#include "RHIResources.h"
#include "Async/Async.h"
#include "RenderingThread.h"
#include "RendererInterface.h"
#include "ColorManagement/ColorSpace.h"

class FMediaPlayerFacade;
class IMediaPlayer;
class IMediaTextureSample;
class UMediaTexture;
struct FGenerateMipsStruct;
struct FPriorSamples;

enum class EMediaTextureSinkFormat;
enum class EMediaTextureSinkMode;

namespace UE::MediaTexture::Private
{
	/** Clamps the number of mips to the maximum needed mips given the texture dimensions. */
	uint8 ClampNumMips(uint8 InNumMips, const FIntPoint& InDim);

	/**
	 * @brief Calculates the target number of generated mips for a given sample.
	 * 
	 * @param InSample Decoded sample.
	 * @param bInEnableGenMips Indicate if mip generation is enabled.
	 * @return Expected number of generated mips.
	 */
	uint8 CalcGeneratedNumMips(const IMediaTextureSample& InSample, bool bInEnableGenMips);

	/**
	 * @brief Calculates the expected texture resource number of mips for the given sample.
	 * Sample source mips take priority over generated ones.
	 *
	 * @param InSample Decoded sample.
	 * @param bInEnableGenMips Indicate if mip generation is enabled.
	 * @return Expected number of mips.
	 */
	uint8 CalcTextureNumMips(const IMediaTextureSample& InSample, bool bInEnableGenMips);
}

/**
 * Texture resource type for media textures.
 */
class FMediaTextureResource
	: public FRenderTarget
	, public FTextureResource
{
public:

	/** 
	 * Creates and initializes a new instance.
	 *
	 * @param InOwner The Movie texture object to create a resource for (must not be nullptr).
	 * @param InOwnerDim Reference to the width and height of the texture that owns this resource (will be updated by resource).
	 * @param InOwnerSize Reference to the size in bytes of the texture that owns this resource (will be updated by resource).
	 * @param InClearColor The initial clear color.
	 * @param InTextureGuid The initial external texture GUID.
	 * @param bEnableGenMips If true mips generation will be enabled (possibly optimizing for NumMips == 1 case)
	 * @param InNumMips The initial number of mips to be generated for the output texture
	 */
	MEDIAASSETS_API FMediaTextureResource(UMediaTexture& InOwner, FIntPoint& InOwnerDim, SIZE_T& InOwnerSize, FLinearColor InClearColor, FGuid InTextureGuid, bool bEnableGenMips, uint8 InNumMips, UE::Color::EColorSpace OverrideColorSpaceType);

	/** Virtual destructor. */
	virtual ~FMediaTextureResource() = default; 

public:

	/** Parameters for the Render method. */
	struct FRenderParams
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FRenderParams() = default;
		FRenderParams(const FRenderParams& InOther) = default;
		FRenderParams(FRenderParams&& InOther) = default;
		FRenderParams& operator=(const FRenderParams&) = default;
		FRenderParams& operator=(FRenderParams&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Whether the texture can be cleared. */
		bool CanClear;

		/** The clear color to use when clearing the texture. */
		FLinearColor ClearColor;

		/** The texture's current external texture GUID. */
		FGuid CurrentGuid;

		/** The texture's previously used external texture GUID. */
		FGuid PreviousGuid;

		/** The player's play rate. */
		float Rate;

		/** The player facade that provides the video samples to render. */
		TWeakPtr<FMediaTextureSampleSource, ESPMode::ThreadSafe> SampleSource;

		/** Number of mips wanted */
		UE_DEPRECATED(5.6, "Unused since UMediaTexture NumMips is now deprecated.")
		uint8 NumMips;

		/** The time of the video frame to render (in player's clock). */
		FMediaTimeStamp Time;

		/** Explicit texture sample to render - if set time will be ignored */
		TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> TextureSample;
	};

	/**
	 * Render the texture resource.
	 *
	 * This method is called on the render thread by the MediaTexture that owns this
	 * texture resource to clear or redraw the resource using the given parameters.
	 *
	 * @param Params Render parameters.
	 */
	void Render(const FRenderParams& Params);

	/**
	 * Flush out any pending data like texture samples waiting for retirement etc.
	 * @note this call can stall for noticable amounts of time under certain circumstances
	 */
	void FlushPendingData();

	/** Sets the just in time render parameters for later use when JustInTimeRender() gets called */
	void SetJustInTimeRenderParams(const FRenderParams& InJustInTimeRenderParams);

	/** Clears the just in time render params, in which case calling JustInTimeRender() would have no effect */
	void ResetJustInTimeRenderParams();

	/** Render the texture using the cached FRenderParams. Call from render thread only. */
	void JustInTimeRender();
	
public:

	//~ FRenderTarget interface

	virtual FIntPoint GetSizeXY() const override;

public:

	//~ FTextureResource interface

	virtual FString GetFriendlyName() const override;
	virtual uint32 GetSizeX() const override;
	virtual uint32 GetSizeY() const override;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

protected:

	/**
	 * Clear the texture using the given clear color.
	 *
	 * @param ClearColor The clear color to use.
	 * @param SrgbOutput Whether the output texture is in sRGB color space.
	 */
	void ClearTexture(FRHICommandListImmediate& RHICmdList, const FLinearColor& ClearColor, bool SrgbOutput);

	/**
	 * Render the given texture sample by converting it on the GPU.
	 *
	 * @param Sample The texture sample to convert.
	 * @param ClearColor The clear color to use for the output texture.
	 * @param Number of mips
	 * @see CopySample
	 */
	void ConvertSample(FRHICommandListImmediate& RHICmdList, const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample, const FLinearColor& ClearColor, uint8 InTargetNumMips);

	void ConvertTextureToOutput(FRHICommandListImmediate& RHICmdList, FRHITexture* InputTexture, const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample);

	/**
	 * Render the given texture sample by using it as or copying it to the render target.
	 *
	 * @param Sample The texture sample to copy.
	 * @param ClearColor The clear color to use for the output texture.
	 * @param SrgbOutput Whether the output texture is in sRGB color space.
	 * @param Number of mips
	 * @see ConvertSample
	 */
	void CopySample(FRHICommandListImmediate& RHICmdList, const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample, const FLinearColor& ClearColor, uint8 InTargetNumMips, const FGuid & TextureGUID);

	/** Calculates the current resource size and notifies the owner texture. */
	void UpdateResourceSize();

	/**
	 * Set the owner's texture reference to the given texture.
	 *
	 * @param NewTexture The texture to set.
	 */
	void UpdateTextureReference(FRHICommandListImmediate& RHICmdList, FRHITexture* NewTexture);

	/**
	 * Create/update intermediate render target as needed. If no color conversion is needed, the RT will be used as the output.
	 *
	 * InExtraCreateFlags - additional ETextureCreateFlags OR'd into the create flags before
	 * the FRHITextureCreateDesc is constructed. Lets a caller request extra capabilities on
	 * the intermediate target (e.g. TexCreate_ReservedResource for sparse residency) without
	 * affecting samples that don't opt in. Default of None preserves existing behavior for
	 * all current callers.
	 */
	void CreateIntermediateRenderTarget(FRHICommandListImmediate& RHICmdList, const FIntPoint & InDim, EPixelFormat InPixelFormat, bool bInSRGB, const FLinearColor & InClearColor, uint8 InNumMips, bool bNeedsUAVSupport, ETextureCreateFlags InExtraCreateFlags = ETextureCreateFlags::None);
	
	/**
	 * Caches next available sample from queue in MediaTexture owner to keep single consumer access
	 *
	 * @param InSampleQueue SampleQueue to query sample information from
	 */
	void CacheNextAvailableSampleTime(const TSharedPtr<FMediaTextureSampleSource, ESPMode::ThreadSafe>& InSampleQueue) const;

	/** Setup sampler state from owner's settings as needed */
	void SetupSampler();

	/** Copy to local buffer from external texture */
	void CopyFromExternalTexture(FRHICommandListImmediate& RHICmdList, const TSharedPtr <IMediaTextureSample, ESPMode::ThreadSafe>& Sample, const FGuid & TextureGUID);

	bool RequiresConversion(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample, uint8 InTargetNumMips) const;
	bool RequiresConversion(const FTextureRHIRef& SampleTexture, const FIntPoint & OutputDim, uint8 InTargetNumMips) const;

	/** Compute CS conversion martix based on sample's data */
	void GetColorSpaceConversionMatrixForSample(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample, FMatrix44f& ColorSpaceMtx);

private:

	/** Platform uses GL/ES ImageExternal */
	bool bUsesImageExternal;

	/** Whether the texture has been cleared. */
	bool Cleared;

	/** Tracks the current clear color. */
	FLinearColor CurrentClearColor;

	/** The external texture GUID to use when initializing this resource. */
	FGuid InitialTextureGuid;

	/** Input render target if the texture samples don't provide one (for conversions). */
	TRefCountPtr<FRHITexture> InputTarget;

	/** Holds the intermediate render target if the texture samples don't provide one, or final render target when not using a texture sample color converter. */
	TRefCountPtr<FRHITexture> IntermediateTarget;

	/** Output render target where the texture sample color converter writes. */
	TRefCountPtr<FRHITexture> OutputTarget;

	/** The media texture that owns this resource. */
	UMediaTexture& Owner;

	/** Reference to the owner's texture dimensions field. */
	FIntPoint& OwnerDim;

	/** Reference to the owner's texture size field. */
	SIZE_T& OwnerSize;

	/** Enable mips generation */
	bool bEnableGenMips;

	/** Current number of mips to be generated as output */
	uint8 CurrentNumMips;

	/** Current texture sampler filter value */
	ESamplerFilter CurrentSamplerFilter;

	/** Current texture sampler mip bias. */
	float CurrentMipMapBias;

	/** The current media player facade to get video samples from. */
	TWeakPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> PlayerFacadePtr;

	/** cached media sample to postpone releasing it until the next sample rendering as it can get overwritten due to asynchronous rendering */
	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> CurrentSample;

	/** prior samples not yet ready for retirement as GPU may still actively use them */
	TSharedRef<FPriorSamples, ESPMode::ThreadSafe> PriorSamples;
	/** prior samples CS */
	FCriticalSection PriorSamplesCS;

	/** cached params etc. for use with mip generator */
	TRefCountPtr<IPooledRenderTarget> MipGenerationCache;

	/** Cached FRenderParams, used when JustInTimeRender() gets called. */
	TUniquePtr<FRenderParams> JustInTimeRenderParams;

	/** Destination colorspace to override standard project "working color space'. Used primarily by Slate which remains sRGB-only. */
	TUniquePtr<UE::Color::FColorSpace> OverrideColorSpace;

	/** Used to keep track of whether we should re-create the output target because the intermediate target has changed. */
	bool bRecreateOutputTarget = false;
};
