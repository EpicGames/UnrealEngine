// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "FileMediaSource.h"
#include "MediaPlaylist.h"
#include "Misc/AutomationTest.h"
#include "UObject/UObjectGlobals.h"

#if WITH_DEV_AUTOMATION_TESTS


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaPlaylistTest, "System.Media.Assets.Playlist", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)


bool FMediaPlaylistTest::RunTest(const FString& Parameters)
{
	auto Playlist = NewObject<UMediaPlaylist>();
	int32 Index;

	// empty play list
	{
		TestEqual(TEXT("A new play list must be empty"), Playlist->Num(), 0);

		Index = INDEX_NONE;
		TestNull(TEXT("GetNext(INDEX_NONE) on a new play list must return nullptr"), Playlist->GetNext(Index));
		TestEqual(TEXT("GetNext(INDEX_NONE) on a new play list must yield INDEX_NONE"), Index, INDEX_NONE);
		TestNull(TEXT("GetPrevious(INDEX_NONE) on a new play list must return nullptr"), Playlist->GetPrevious(Index));
		TestEqual(TEXT("GetPrevious(INDEX_NONE) on a new play list must yield INDEX_NONE"), Index, INDEX_NONE);

		Index = 0;
		TestNull(TEXT("GetNext(0) on a new play list must return nullptr"), Playlist->GetNext(Index));
		TestEqual(TEXT("GetNext(0) on a new play list must yield INDEX_NONE"), Index, INDEX_NONE);
		TestNull(TEXT("GetPrevious(0) on a new play list must return nullptr"), Playlist->GetPrevious(Index));
		TestEqual(TEXT("GetPrevious(0) on a new play list must yield INDEX_NONE"), Index, INDEX_NONE);

		Index = 1;
		TestNull(TEXT("GetNext(1) on a new play list must return nullptr"), Playlist->GetNext(Index));
		TestEqual(TEXT("GetNext(1) on a new play list must yield INDEX_NONE"), Index, INDEX_NONE);
		TestNull(TEXT("GetPrevious(1) on a new play list must return nullptr"), Playlist->GetPrevious(Index));
		TestEqual(TEXT("GetPrevious(1) on a new play list must yield INDEX_NONE"), Index, INDEX_NONE);

		TestNull(TEXT("GetRandom() on a new play list must return nullptr"), Playlist->GetRandom(Index));
		TestEqual(TEXT("GetRandom() on a new play list must yield INDEX_NONE"), Index, INDEX_NONE);
	}

	// one entry
	UMediaSource* MediaSource1 = NewObject<UFileMediaSource>();
	{
		Playlist->Add(MediaSource1);
		TestEqual(TEXT("A play list with one entry and must have length 1"), Playlist->Num(), 1);

		Index = INDEX_NONE;
		TestNotNull(TEXT("GetNext(INDEX_NONE) on a play list with one entry must return the first item"), Playlist->GetNext(Index));
		TestEqual(TEXT("GetNext(INDEX_NONE) on a play list with one entry must yield 0"), Index, 0);

		Index = INDEX_NONE;
		TestNotNull(TEXT("GetPrevious(INDEX_NONE) on a play list with one entry must return the first item"), Playlist->GetPrevious(Index));
		TestEqual(TEXT("GetPrevious(INDEX_NONE) on a play list with one entry must yield 0"), Index, 0);

		Index = 0;
		TestNotNull(TEXT("GetNext(0) on a play list with one entry must return the first item"), Playlist->GetNext(Index));
		TestEqual(TEXT("GetNext(0) on a play list with one entry must yield 0"), Index, 0);

		Index = 0;
		TestNotNull(TEXT("GetPrevious(0) on a play list with one entry must return the first item"), Playlist->GetPrevious(Index));
		TestEqual(TEXT("GetPrevious(0) on a play list with one entry must yield 0"), Index, 0);

		Index = 1;
		TestNotNull(TEXT("GetNext(1) on a play list with one entry must return the first item"), Playlist->GetNext(Index));
		TestEqual(TEXT("GetNext(1) on a play list with one entry must yield 0"), Index, 0);

		Index = 1;
		TestNotNull(TEXT("GetPrevious(1) on a play list with one entry must return the first item"), Playlist->GetPrevious(Index));
		TestEqual(TEXT("GetPrevious(1) on a play list with one entry must yield 0"), Index, 0);
	}

	// two entries
	UMediaSource* MediaSource2 = NewObject<UFileMediaSource>();
	{
		Playlist->Add(MediaSource2);
		TestEqual(TEXT("A play list with two entries and must have length 2"), Playlist->Num(), 2);

		Index = INDEX_NONE;
		TestEqual(TEXT("GetNext(INDEX_NONE) on a play list with two entries must return the first item"), Playlist->GetNext(Index), MediaSource1);
		TestEqual(TEXT("GetNext(INDEX_NONE) on a play list with two entries must yield 0"), Index, 0);

		Index = INDEX_NONE;
		TestEqual(TEXT("GetPrevious(INDEX_NONE) on a play list with two entries must return the second item"), Playlist->GetPrevious(Index), MediaSource2);
		TestEqual(TEXT("GetPrevious(INDEX_NONE) on a play list with two entries must yield 1"), Index, 1);

		Index = 0;
		TestEqual(TEXT("GetNext(0) on a play list with two entries must return the second item"), Playlist->GetNext(Index), MediaSource2);
		TestEqual(TEXT("GetNext(0) on a play list with two entries must yield 1"), Index, 1);

		Index = 0;
		TestEqual(TEXT("GetPrevious(0) on a play list with two entries must return the first item"), Playlist->GetPrevious(Index), MediaSource2);
		TestEqual(TEXT("GetPrevious(0) on a play list with two entries must yield 1"), Index, 1);

		Index = 1;
		TestEqual(TEXT("GetNext(1) on a play list with two entries must return the first item"), Playlist->GetNext(Index), MediaSource1);
		TestEqual(TEXT("GetNext(1) on a play list with two entries must yield 0"), Index, 0);

		Index = 1;
		TestEqual(TEXT("GetPrevious(1) on a play list with two entries must return the first item"), Playlist->GetPrevious(Index), MediaSource1);
		TestEqual(TEXT("GetPrevious(1) on a play list with two entries must yield 0"), Index, 0);

		TestNotNull(TEXT("GetRandom() on a play list with two entries and must not return nullptr"), Playlist->GetRandom(Index));
		TestNotEqual(TEXT("GetRandom() on a play list with two entries and must not yield INDEX_NONE"), Index, (int32)INDEX_NONE);
	}

	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS
