// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "MediaSourcePathValidation.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaSourcePathValidationTest, "System.Media.Assets.PathValidation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)


bool FMediaSourcePathValidationTest::RunTest(const FString& Parameters)
{
	using namespace UE::MediaAssets;

	// IsUNCPath: trivial non-UNC inputs
	TestFalse(TEXT("Empty path is not UNC"), IsUNCPath(FStringView()));
	TestFalse(TEXT("Single backslash is not UNC"), IsUNCPath(TEXTVIEW("\\foo")));
	TestFalse(TEXT("Single forward slash is not UNC"), IsUNCPath(TEXTVIEW("/foo")));
	TestFalse(TEXT("Drive-letter path is not UNC"), IsUNCPath(TEXTVIEW("C:\\foo")));
	TestFalse(TEXT("Relative dot path is not UNC"), IsUNCPath(TEXTVIEW("./content/foo")));

	// IsUNCPath: standard UNC forms
	TestTrue(TEXT("Backslash UNC is detected"), IsUNCPath(TEXTVIEW("\\\\server\\share")));
	TestTrue(TEXT("Forward-slash UNC is detected"), IsUNCPath(TEXTVIEW("//server/share")));

	// IsUNCPath: Win32 namespace UNCs
	TestTrue(TEXT("Win32 file namespace UNC is detected"),
		IsUNCPath(TEXTVIEW("\\\\?\\UNC\\server\\share")));
	TestTrue(TEXT("Win32 device namespace UNC is detected"),
		IsUNCPath(TEXTVIEW("\\\\.\\UNC\\server\\share")));
	TestTrue(TEXT("Lowercase 'unc' segment in namespace UNC is detected"),
		IsUNCPath(TEXTVIEW("\\\\?\\unc\\server\\share")));

	// IsUNCPath: namespace prefixes that are NOT UNC
	TestFalse(TEXT("Long-path local form is not UNC"),
		IsUNCPath(TEXTVIEW("\\\\?\\C:\\path")));
	TestFalse(TEXT("Device path (COM1) is not UNC"),
		IsUNCPath(TEXTVIEW("\\\\.\\COM1")));

	// IsUNCPath: file:// URI stripping
	TestTrue(TEXT("file:// + backslash UNC is detected"),
		IsUNCPath(TEXTVIEW("file://\\\\server\\share")));
	TestTrue(TEXT("FILE:// + backslash UNC (mixed case) is detected"),
		IsUNCPath(TEXTVIEW("FILE://\\\\server\\share")));
	TestTrue(TEXT("file:// + forward UNC is detected"),
		IsUNCPath(TEXTVIEW("file:////server/share")));
	TestFalse(TEXT("file:// + drive-letter path is not UNC"),
		IsUNCPath(TEXTVIEW("file:///C:/foo")));
	TestFalse(TEXT("file:// alone is not UNC"),
		IsUNCPath(TEXTVIEW("file://")));

	// IsSourcePathAllowed: non-UNC paths are always allowed
	TestTrue(TEXT("Empty path is allowed"), IsSourcePathAllowed(FStringView()));
	TestTrue(TEXT("Drive-letter path is allowed"),
		IsSourcePathAllowed(TEXTVIEW("C:\\foo")));
	TestTrue(TEXT("Relative path is allowed"),
		IsSourcePathAllowed(TEXTVIEW("./content/sequence.exr")));

	// IsSourcePathAllowed: UNC behavior depends on the read-only cvar. We don't
	// toggle it here (ECVF_ReadOnly), but assert the result matches the current
	// configuration so the gate is exercised in both default and hardened builds.
	const bool bExpectUNCAllowed = AreNetworkPathsAllowed();
	TestEqual(TEXT("UNC path allowance matches AreNetworkPathsAllowed()"),
		IsSourcePathAllowed(TEXTVIEW("\\\\server\\share")), bExpectUNCAllowed);
	TestEqual(TEXT("Namespace UNC allowance matches AreNetworkPathsAllowed()"),
		IsSourcePathAllowed(TEXTVIEW("\\\\?\\UNC\\server\\share")), bExpectUNCAllowed);
	TestEqual(TEXT("file:// + UNC allowance matches AreNetworkPathsAllowed()"),
		IsSourcePathAllowed(TEXTVIEW("file://\\\\server\\share")), bExpectUNCAllowed);

	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS
