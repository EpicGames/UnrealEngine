// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "MediaSource.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BaseMediaSource.generated.h"

class FArchive;
class FObjectPreSaveContext;
class FString;
class UObject;
struct FGuid;


/**
 * Base class for concrete media sources.
 */
UCLASS(Abstract, BlueprintType, hidecategories=(Object), MinimalAPI)
class UBaseMediaSource
	: public UMediaSource
{
	GENERATED_BODY()

public:

#if WITH_EDITORONLY_DATA

	/** Override native media player plug-ins per platform (Empty = find one automatically). */
	UPROPERTY(transient, BlueprintReadWrite, EditAnywhere, Category=Platforms, Meta=(DisplayName="Player Overrides"))
	TMap<FString, FName> PlatformPlayerNames;

private:
	/** Platform to player plugin GUID mappings that could not be resolved on load (e.g. missing platform support) */
	TMap<FGuid, FGuid> BlindPlatformGuidPlayerNames;

#endif

public:

	//~ UObject interface
	MEDIAASSETS_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext);
	MEDIAASSETS_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	MEDIAASSETS_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	MEDIAASSETS_API virtual void Serialize(FArchive& Ar) override;

public:

	//~ IMediaOptions interface

	MEDIAASSETS_API virtual FName GetDesiredPlayerName() const override;

private:

	/** Name of the desired native media player (Empty = find one automatically). */
	UPROPERTY(transient)
	FName PlayerName;
};
