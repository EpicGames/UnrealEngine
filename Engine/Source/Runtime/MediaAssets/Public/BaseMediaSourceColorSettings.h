// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ColorManagement/ColorManagementDefines.h"
#include "ColorManagement/ColorSpace.h"
#include "Engine/TextureDefines.h"
#include "IMediaOptions.h"

#include "BaseMediaSourceColorSettings.generated.h"

/** List of source encodings that can be converted to linear. (Integer values match the ETextureSourceEncoding values in TextureDefines.h */
UENUM()
enum class EMediaSourceEncoding : uint8
{
	MSE_None	= 0 UMETA(DisplayName = "None", ToolTip = "The source encoding is not overridden."),
	MSE_Linear	= 1 UMETA(DisplayName = "Linear", ToolTip = "The source encoding is considered linear."),
	MSE_sRGB	= 2 UMETA(DisplayName = "sRGB", ToolTip = "sRGB source encoding to be linearized"),
	MSE_ST2084	= 3 UMETA(DisplayName = "ST 2084/PQ", ToolTip = "SMPTE ST 2084/PQ source encoding to be linearized"),
	MSE_SLog3	= 12 UMETA(DisplayName = "SLog3", ToolTip = "Sony SLog3 source encoding to be linearized"),
	MSE_HLG		= 17 UMETA(DisplayName = "HLG", ToolTip = "HLG (BT.2100): linearized to the [0, 1] range."),

	MSE_MAX		= 18,
};

/**
 * Reference-white standards to scale HDR media sources so diffuse
 * white lands at UE scene-linear 1.0, and vice-versa.
 */
UENUM()
enum class EMediaReferenceWhite : uint8
{
	MRW_None					= 0 UMETA(DisplayName = "None", ToolTip = "The source reference white is not overridden, defer to system/default behavior."),
	MRW_DisableNormalization	= 1 UMETA(DisplayName = "Disable Normalization", ToolTip = "Explicitly preserve input/native scaling."),
	MRW_BT2408					= 2 UMETA(DisplayName = "BT.2408 (203 nits)", ToolTip = "ITU-R BT.2408 Reference White. 203 cd/m2 for PQ (ST2084), 75% signal for HLG scale to scene linear white (1.0)."),
	MRW_BT1886					= 3 UMETA(DisplayName = "BT.1886 (100 nits)", ToolTip = "ITU-R BT.1886 SDR reference white (100 cd/m2). Scales PQ (100-nit paper white) and HLG (in-house 100-nit convention on a 1000 cd/m2 BT.2100 reference display) to scene linear white (1.0)."),

	MRW_MAX						= 4 UMETA(Hidden),
};

/* Manual definition of media source color space & encoding. */
USTRUCT(BlueprintType)
struct FMediaSourceColorSettings
{
	GENERATED_USTRUCT_BODY()

	FMediaSourceColorSettings()
		: EncodingOverride(EMediaSourceEncoding::MSE_None)
		, ColorSpaceOverride(ETextureColorSpace::TCS_None)
		, RedChromaticityCoordinate(FVector2D::ZeroVector)
		, GreenChromaticityCoordinate(FVector2D::ZeroVector)
		, BlueChromaticityCoordinate(FVector2D::ZeroVector)
		, WhiteChromaticityCoordinate(FVector2D::ZeroVector)
		, ChromaticAdaptationMethod(static_cast<ETextureChromaticAdaptationMethod>(UE::Color::DEFAULT_CHROMATIC_ADAPTATION_METHOD))
		, ReferenceWhiteOverride(EMediaReferenceWhite::MRW_None)
	{}

	/** Source encoding of the media. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement)
	EMediaSourceEncoding EncodingOverride;

	/** Source color space of the media. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement)
	ETextureColorSpace ColorSpaceOverride;

	/** Red chromaticity coordinate of the source color space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement, meta = (EditCondition = "ColorSpaceOverride == ETextureColorSpace::TCS_Custom", EditConditionHides))
	FVector2D RedChromaticityCoordinate;

	/** Green chromaticity coordinate of the source color space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement, meta = (EditCondition = "ColorSpaceOverride == ETextureColorSpace::TCS_Custom", EditConditionHides))
	FVector2D GreenChromaticityCoordinate;

	/** Blue chromaticity coordinate of the source color space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement, meta = (EditCondition = "ColorSpaceOverride == ETextureColorSpace::TCS_Custom", EditConditionHides))
	FVector2D BlueChromaticityCoordinate;

	/** White chromaticity coordinate of the source color space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement, meta = (EditCondition = "ColorSpaceOverride == ETextureColorSpace::TCS_Custom", EditConditionHides))
	FVector2D WhiteChromaticityCoordinate;

	/** Chromatic adaption method applied if the source white point differs from the working color space white point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement, meta = (EditCondition = "ColorSpaceOverride != ETextureColorSpace::TCS_None"))
	ETextureChromaticAdaptationMethod ChromaticAdaptationMethod;

	// TODO: Unify with FTextureSourceColorSettings.
	/** Reference HDR white standard used to scale input signals to scene linear white (1.0). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorManagement, meta = (EditCondition = "EncodingOverride == EMediaSourceEncoding::MSE_ST2084 || EncodingOverride == EMediaSourceEncoding::MSE_HLG", EditConditionHides))
	EMediaReferenceWhite ReferenceWhiteOverride;

#if WITH_EDITOR
	/** Update the chromaticity coordinates member variables based on the color space choice (unless custom). */
	MEDIAASSETS_API void UpdateColorSpaceChromaticities();
#endif
};

/* Engine-native color source settings container for media option. */
struct FNativeMediaSourceColorSettings : public IMediaOptions::FDataContainer
{
	/** Constructor */
	MEDIAASSETS_API FNativeMediaSourceColorSettings();

	/** Destructor */
	MEDIAASSETS_API ~FNativeMediaSourceColorSettings();

	/** Copy constructor */
	MEDIAASSETS_API FNativeMediaSourceColorSettings(const FNativeMediaSourceColorSettings& Other);

	/** Assignment operator */
	MEDIAASSETS_API FNativeMediaSourceColorSettings& operator=(const FNativeMediaSourceColorSettings& Other);

	/** Updates the native settings from user-controlled settings. */
	MEDIAASSETS_API void Update(const FMediaSourceColorSettings& InSettings);

	/** Has encoding override. */
	inline bool HasEncodingOverride() const { return EncodingOverride != UE::Color::EEncoding::None; }

	/** Color encoding override getter. */
	inline UE::Color::EEncoding GetEncodingOverride() const { return EncodingOverride; }

	/** Color encoding override setter. */
	inline void SetEncodingOverride(UE::Color::EEncoding InEncodingOverride) { EncodingOverride = InEncodingOverride; }

	/** Reference-white override getter. See FMediaSourceColorSettings::ReferenceWhiteOverride for semantics. */
	inline UE::Color::EReferenceWhite GetReferenceWhiteOverride() const { return ReferenceWhiteOverride; }

	/** Reference-white override setter. */
	inline void SetReferenceWhiteOverride(UE::Color::EReferenceWhite InReferenceWhiteOverride) { ReferenceWhiteOverride = InReferenceWhiteOverride; }

	/** Has color space override. */
	MEDIAASSETS_API bool HasColorSpaceOverride() const;

	/** Color space override getter. */
	MEDIAASSETS_API const UE::Color::FColorSpace& GetColorSpaceOverride(const UE::Color::FColorSpace& InDefaultColorSpace) const;

	/** Color space override setter. */
	MEDIAASSETS_API void SetColorSpaceOverride(const UE::Color::FColorSpace& InColorSpaceOverride);

	/** Chromatic adaptation getter. */
	inline UE::Color::EChromaticAdaptationMethod GetChromaticAdaptationMethod() const { return ChromaticAdaptationMethod; }

private:
	/** Manual source encoding override. */
	std::atomic<UE::Color::EEncoding> EncodingOverride;

	/** Companion reference-white override. EReferenceWhite::None means no normalization. */
	std::atomic<UE::Color::EReferenceWhite> ReferenceWhiteOverride;

	/** Manual source color space override. */
	TOptional<UE::Color::FColorSpace> ColorSpaceOverride;

	/** Chromatic adapation to be used on manual source color space override. */
	std::atomic<UE::Color::EChromaticAdaptationMethod> ChromaticAdaptationMethod;

	/** Protects color space override variable. */
	mutable FCriticalSection ColorSpaceCriticalSection;
};
