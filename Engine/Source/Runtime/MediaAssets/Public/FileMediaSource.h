// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMediaSource.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "FileMediaSource.generated.h"

class UObject;
struct FFrame;
struct FPropertyChangedEvent;


UCLASS(BlueprintType, MinimalAPI)
class UFileMediaSource
	: public UBaseMediaSource
{
	GENERATED_BODY()

public:

	/**
	 * The path to the media file to be played.
	 *
	 * @see SetFilePath
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=File, AssetRegistrySearchable)
	FString FilePath;


	/** Load entire media file into memory and play from there (if possible). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=File, AdvancedDisplay)
	bool PrecacheFile;

public:

	/**
	 * Get the path to the media file to be played.
	 *
	 * @return The file path.
	 * @see GetFullPath, SetFilePath
	 */
	const FString& GetFilePath() const
	{
		return FilePath;
	}

	/**
	 * Get the full path to the file.
	 *
	 * @return The full file path.
	 * @return GetFilePath
	 */
	MEDIAASSETS_API FString GetFullPath() const;

	/**
	 * Set the path to the media file that this source represents.
	 *
	 * Automatically converts full paths to media sources that reside in the
	 * Engine's or project's /Content/Movies directory into relative paths.
	 *
	 * @param Path The path to set.
	 * @see FilePath, GetFilePath
	 */
	UFUNCTION(BlueprintCallable, Category="Media|FileMediaSource")
	MEDIAASSETS_API void SetFilePath(const FString& Path);

#if WITH_EDITOR
	MEDIAASSETS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:

	//~ IMediaOptions interface

	MEDIAASSETS_API virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	MEDIAASSETS_API virtual bool HasMediaOption(const FName& Key) const override;

public:

	//~ UMediaSource interface

	MEDIAASSETS_API virtual FString GetUrl() const override;
	MEDIAASSETS_API virtual bool Validate() const override;

#if WITH_EDITOR
	MEDIAASSETS_API virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif
public:

	/** Name of the PrecacheFile media option. */
	static MEDIAASSETS_API FName PrecacheFileOption;

private:
	MEDIAASSETS_API void ResolveFullPath() const;
	MEDIAASSETS_API void ClearResolvedFullPath() const;
	mutable FString ResolvedFullPath; // this is a cached variable updated in ResolveFullPath hence mutable
};
