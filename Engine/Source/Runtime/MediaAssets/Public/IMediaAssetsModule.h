// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

class IMediaSourceRendererInterface;
class UMediaPlayer;
class UObject;

/**
 * Interface for the MediaAssets module.
 */
class IMediaAssetsModule
	: public IModuleInterface
{
public:
	/** Delegate to get a player from a UObject. */
	DECLARE_DELEGATE_RetVal_TwoParams(UMediaPlayer*, FOnGetPlayerFromObject, UObject*, UObject*& /*PlayerProxy*/);
	/** Delegate to create an object that implements IMediaSourceRendererInterface. */
	DECLARE_DELEGATE_RetVal(UObject*, FOnCreateMediaSourceRenderer);
	/** Delegate that reacts to change in Media state (Play, Stop, Pause etc.) */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FMediaStateChangedDelegate, const TArray<FString>& /*InActorsPathNames*/, uint8 /*EnumState*/, bool /*bRemoteBroadcast*/);

	/**
	 * Plugins should call this so they can provide a function to get a media player from an object.
	 * 
	 * @param Delegate		Delegate to get a media player.
	 * @return ID to pass in to UnregisterGetPlayerFromObject.
	 */
	virtual int32 RegisterGetPlayerFromObject(const FOnGetPlayerFromObject& Delegate) = 0;

	/**
	 * Call this to unregister a delegate.
	 *
	 * @param DelegateID	ID returned from RegisterGetPlayerFromObject.
	 */
	virtual void UnregisterGetPlayerFromObject(int32 DelegateID) = 0;

	/**
	 * Call this to get a media player (and proxy object) from an object.
	 * This will query any plugins that have called RegisterGetPlayerFromObject.
	 * 
	 * The proxy object will implement IMediaPlayerProxyInterface.
	 * 
	 * @param Object		Object to get the player from.
	 * @param PlayerProxy	Will be set to the proxy object (or nullptr if none).
	 * @return Media player, or nullptr if none found. 
	 */
	virtual UMediaPlayer* GetPlayerFromObject(UObject* Object, UObject*& PlayerProxy) = 0;

	/**
	 * Register a delegate to create an object that implements IMediaSourceRendererInterface.
	 */
	virtual void RegisterCreateMediaSourceRenderer(const FOnCreateMediaSourceRenderer& Delegate) = 0;
	
	/**
	 * Unregisters the delegate passed in with RegisterCreateMediaSourceRenderer.
	 */
	virtual void UnregisterCreateMediaSourceRenderer() = 0;

	/** Subscribes to the event that is called whenever any of Media state changes (such as play button was pressed). */
	virtual FDelegateHandle RegisterOnMediaStateChangedEvent(FMediaStateChangedDelegate::FDelegate InStateChangedDelegate) = 0;

	/** Removes event handling. */
	virtual void UnregisterOnMediaStateChangedEvent(FDelegateHandle InHandle) = 0;

	/* Broadcasts the event to all subscribers. InActorsPathNames are the paths of selected Medias in the scene. */
	virtual void BroadcastOnMediaStateChangedEvent(const TArray<FString>& InActorsPathNames, uint8 EnumState, bool bRemoteBroadcast = false) = 0;

	/**
	 * Creates an object that implements IMediaSourceRendererInterface.
	 */
	virtual UObject* CreateMediaSourceRenderer() = 0;

	/** Virtual destructor. */
	virtual ~IMediaAssetsModule() = default;
};
