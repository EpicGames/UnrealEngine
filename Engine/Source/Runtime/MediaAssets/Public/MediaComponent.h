// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "MediaComponent.generated.h"

class UMediaPlayer;
class UMediaTexture;

UCLASS(Category="Media", hideCategories=(Media, Activation), MinimalAPI)
class UMediaComponent : public UActorComponent
{
public:
	GENERATED_BODY()

	MEDIAASSETS_API UMediaComponent(const FObjectInitializer& ObjectInitializer);

	/** Get the media player that this component owns */
	UFUNCTION(BlueprintPure, Category="Media")
	MEDIAASSETS_API UMediaPlayer* GetMediaPlayer() const;

	/** Get the media texture that this component owns, bound to its media player. */
	UFUNCTION(BlueprintPure, Category="Media")
	MEDIAASSETS_API UMediaTexture* GetMediaTexture() const;

private:

	MEDIAASSETS_API virtual void OnRegister() override;

private:

	/** This component's media texture */
	UPROPERTY(Category="Media", Instanced, Transient, BlueprintGetter=GetMediaTexture)
	TObjectPtr<UMediaTexture> MediaTexture;

	/** This component's media player */
	UPROPERTY(Category="Media", Instanced, Transient, BlueprintGetter=GetMediaPlayer, Interp)
	TObjectPtr<UMediaPlayer> MediaPlayer;
};
