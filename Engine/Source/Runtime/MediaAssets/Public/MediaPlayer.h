// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "IMediaOptions.h"
#include "IMediaMetadataItem.h"
#include "IMediaControls.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Misc/Guid.h"
#include "Misc/Variant.h"
#include "Misc/Timecode.h"
#include "Engine/LatentActionManager.h"
#include "MediaPlayerOptions.h"
#include "IMediaTimeSource.h"
#include "IAudioProxyInitializer.h"

#include "MediaPlayer.generated.h"

class FMediaPlayerFacade;
class IMediaPlayer;
class IMediaModule;
class IMediaMetadataItem;
class UMediaPlaylist;
class UMediaSource;
class UMediaPlayer;

enum class EMediaEvent;


/** Multicast delegate that is invoked when a media event occurred in the player. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnMediaPlayerMediaEvent);

/** Multicast delegate that is invoked when a media player's media has been opened. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMediaPlayerMediaOpened, FString, OpenedUrl);

/** Multicast delegate that is invoked when a media player's media has failed to open. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMediaPlayerMediaOpenFailed, FString, FailedUrl);


/**
 * Media track types.
 *
 * Note: Keep this in sync with EMediaTrackType
 */
UENUM(BlueprintType)
enum class EMediaPlayerTrack : uint8
{
	/** Audio track. */
	Audio,

	/** Caption track. */
	Caption,

	/** Metadata track. */
	Metadata,

	/** Script track. */
	Script,

	/** Subtitle track. */
	Subtitle,

	/** Text track. */
	Text,

	/** Video track. */
	Video
};


/**
 * Blueprint usable enum of EMediaTimeRangeType
 */
UENUM(BlueprintType)
enum class EMediaTimeRangeBPType : uint8
{
	/** Total absolute time range as defined by the media. */
	Absolute = (uint8)EMediaTimeRangeType::Absolute,

	/** Current time range of the media, set by media internal means or through API calls. */
	Current = (uint8)EMediaTimeRangeType::Current

};


UCLASS(BlueprintType, hidecategories = (Object), MinimalAPI)
class UMediaTimeStampInfo
	: public UObject
{
	GENERATED_BODY()

public:
	UMediaTimeStampInfo() : Time(FTimespan::Zero()), SequenceIndex(0) {}

	UPROPERTY(BlueprintReadOnly, Category = "Media|Time")
	FTimespan Time;

	UPROPERTY(BlueprintReadOnly, Category = "Media|Time")
	int64 SequenceIndex;
};

USTRUCT(BlueprintType)
struct FMediaMetadataItemBPT
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = "Media|Metadata")
	FString LanguageCode;

	UPROPERTY(BlueprintReadOnly, Category = "Media|Metadata")
	FString MimeType;

	UPROPERTY(BlueprintReadOnly, Category = "Media|Metadata")
	FString StringData;

	UPROPERTY(BlueprintReadOnly, Category = "Media|Metadata")
	TArray<uint8> BinaryData;
};

USTRUCT(BlueprintType)
struct FMediaMetadataItemsBPT
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadOnly, Category = "Media|Metadata")
	TArray<FMediaMetadataItemBPT> Items;
};

/**
 * Proxy implementation for Metasound integration
 */
class FMediaPlayerProxy : public Audio::TProxyData<FMediaPlayerProxy>
{
public:
	IMPL_AUDIOPROXY_CLASS(FMediaPlayerProxy);

	MEDIAASSETS_API explicit FMediaPlayerProxy(UMediaPlayer* Player);

	MEDIAASSETS_API FMediaPlayerProxy(const FMediaPlayerProxy& Other);

	MEDIAASSETS_API ~FMediaPlayerProxy();

	TSharedPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> GetPlayerFacade() const
	{
		return(PlayerFacade.Pin());
	}

	friend inline uint32 GetTypeHash(const FMediaPlayerProxy& InProxy)
	{
		return InProxy.TypeHash;
	}

private:
	TWeakPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> PlayerFacade;
	uint32 TypeHash = INDEX_NONE;
};
using FMediaPlayerProxyPtr = TSharedPtr<FMediaPlayerProxy, ESPMode::ThreadSafe>;


/**
 * Implements a media player asset that can play movies and other media sources.
 */
UCLASS(BlueprintType, hidecategories=(Object), MinimalAPI)
class UMediaPlayer
	: public UObject
	, public IAudioProxyDataFactory
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Check whether media playback can be paused right now.
	 *
	 * Playback can be paused if the media supports pausing and if it is currently playing.
	 *
	 * @return true if pausing playback can be paused, false otherwise.
	 * @see CanPlay, Pause
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool CanPause() const;

	/**
	 * Check whether the specified media source can be played by this player.
	 *
	 * If a desired player name is set for this player, it will only check
	 * whether that particular player type can play the specified source.
	 *
	 * @param MediaSource The media source to check.
	 * @return true if the media source can be opened, false otherwise.
	 * @see CanPlayUrl, SetDesiredPlayerName
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API bool CanPlaySource(UMediaSource* MediaSource);

	/**
	 * Check whether the specified URL can be played by this player.
	 *
	 * If a desired player name is set for this player, it will only check
	 * whether that particular player type can play the specified URL.
	 *
	 * @param Url The URL to check.
	 * @see CanPlaySource, SetDesiredPlayerName
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool CanPlayUrl(const FString& Url);

	/**
	 * Close the currently open media, if any.
	 *
	 * @see OnMediaClosed, OpenPlaylist, OpenPlaylistIndex, OpenSource, OpenUrl, Pause, Play
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API void Close();

	/**
	 * Get the number of channels in the specified audio track.
	 *
	 * @param TrackIndex Index of the audio track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Number of channels.
	 * @see GetAudioTrackSampleRate, GetAudioTrackType
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API int32 GetAudioTrackChannels(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the sample rate of the specified audio track.
	 *
	 * @param TrackIndex Index of the audio track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Samples per second.
	 * @see GetAudioTrackChannels, GetAudioTrackType
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API int32 GetAudioTrackSampleRate(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the type of the specified audio track format.
	 *
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Audio format type string.
	 * @see GetAudioTrackSampleRate, GetAudioTrackSampleRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FString GetAudioTrackType(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the name of the current desired native player.
	 *
	 * @return The name of the desired player, or NAME_None if not set.
	 * @see SetDesiredPlayerName
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API FName GetDesiredPlayerName() const;

	/**
	 * Get the media's duration.
	 *
	 * @return A time span representing the duration.
	 * @see GetTime, Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FTimespan GetDuration() const;

	/**
	 * Get the current horizontal field of view (only for 360 videos).
	 *
	 * @return Horizontal field of view (in Euler degrees).
	 * @see GetVerticalFieldOfView, GetViewRotation, SetHorizontalFieldOfView
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API float GetHorizontalFieldOfView() const;

	/**
	 * Get the human readable name of the currently loaded media source.
	 *
	 * @return Media source name, or empty text if no media is opened
	 * @see GetPlayerName, GetUrl
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API virtual FText GetMediaName() const;

	/**
	 * Get the number of tracks of the given type.
	 *
	 * @param TrackType The type of media tracks.
	 * @return Number of tracks.
	 * @see GetNumTrackFormats, GetSelectedTrack, SelectTrack
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API int32 GetNumTracks(EMediaPlayerTrack TrackType) const;

	/**
	 * Get the number of formats of the specified track.
	 *
	 * @param TrackType The type of media tracks.
	 * @param TrackIndex The index of the track.
	 * @return Number of formats.
	 * @see GetNumTracks, GetSelectedTrack, SelectTrack
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API int32 GetNumTrackFormats(EMediaPlayerTrack TrackType, int32 TrackIndex) const;

	/**
	 * Get the name of the current native media player.
	 *
	 * @return Player name, or NAME_None if not available.
	 * @see GetMediaName
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FName GetPlayerName() const;

	/**
	 * Get the current play list.
	 *
	 * Media players always have a valid play list. In C++ code you can use
	 * the GetPlaylistRef to get a reference instead of a pointer to it.
	 *
	 * @return The play list.
	 * @see GetPlaylistIndex, GetPlaylistRef
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	UMediaPlaylist* GetPlaylist() const
	{
		EnsurePlaylist();
		return Playlist;
	}

	/**
	 * Get the current play list index.
	 *
	 * @return Play list index.
	 * @see GetPlaylist
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	int32 GetPlaylistIndex() const
	{
		return PlaylistIndex;
	}

	/**
	 * Get the media's current playback rate.
	 *
	 * @return The playback rate.
	 * @see SetRate, SupportsRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API float GetRate() const;

	/**
	 * Get the index of the currently selected track of the given type.
	 *
	 * @param TrackType The type of track to get.
	 * @return The index of the selected track, or INDEX_NONE if no track is active.
	 * @see GetNumTracks, GetTrackFormat, SelectTrack
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API int32 GetSelectedTrack(EMediaPlayerTrack TrackType) const;

	/**
	 * Get the supported playback rates.
	 *
	 * @param Unthinned Whether the rates are for unthinned playback.
	 * @param OutRates Will contain the the ranges of supported rates.
	 * @see SetRate, SupportsRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API void GetSupportedRates(TArray<FFloatRange>& OutRates, bool Unthinned) const;

	/**
	 * Get the media's current playback time.
	 *
	 * @return Playback time.
	 * @see GetDuration, Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FTimespan GetTime() const;

	/**
	 * Get the media's current playback time as appropriate for display.
	 *
	 * @return Playback time.
	 * @see GetDuration, Seek
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API FTimespan GetDisplayTime() const;

	/**
	 * Get the media's current playback timestamp.
	 *
	 * @return Playback timestamp.
	 * @see GetDuration, Seek
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API UMediaTimeStampInfo* GetTimeStamp() const;

	/**
	 * Get the media's current playback timestamp as appropriate for display.
	 *
	 * @return Playback timestamp.
	 * @see GetDuration, Seek
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API UMediaTimeStampInfo* GetDisplayTimeStamp() const;

	/**
	 * Get the timecode from the video track, if one is available.
	 * If available, the returned value will be the timecode of the sample most recently
	 * sent into the video sink.
	 *
	 * @return Most recently delivered video timecode from the video track.
	 *
	 * @note Even if timecode is available, the value returned reflects the timecode
	 *       that most recently delivered to the video sink. During a seek operation
	 *       the timecode will be unset.
	 */
	MEDIAASSETS_API TOptional<FTimecode> GetVideoTimecode() const;

	/**
	 * Gets the duration of a specific track, if available.
	 * Availability depends on the media format and player implementation.
	 * If unset only the overall media duration can be used.
	 *
	 * @return An optional time span representing the duration.
	 * @see GetDuration
	 */
	MEDIAASSETS_API TOptional<FTimespan> GetTrackDuration(EMediaPlayerTrack TrackType, int32 TrackIndex) const;

	/**
	 * Get the human readable name of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @return Display name.
	 * @see GetNumTracks, GetTrackLanguage
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API FText GetTrackDisplayName(EMediaPlayerTrack TrackType, int32 TrackIndex) const;

	/**
	 * Get the index of the active format of the specified track type.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @return The index of the selected format.
	 * @see GetNumTrackFormats, GetSelectedTrack, SetTrackFormat
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API int32 GetTrackFormat(EMediaPlayerTrack TrackType, int32 TrackIndex) const;

	/**
	 * Get the language tag of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @return Language tag, i.e. "en-US" for English, or "und" for undefined.
	 * @see GetNumTracks, GetTrackDisplayName
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FString GetTrackLanguage(EMediaPlayerTrack TrackType, int32 TrackIndex) const;

	/**
	 * Get the URL of the currently loaded media, if any.
	 *
	 * @return Media URL, or empty string if no media was loaded.
	 * @see OpenUrl
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API const FString& GetUrl() const;

	/**
	 * Get the current vertical field of view (only for 360 videos).
	 *
	 * @return Vertical field of view (in Euler degrees), or 0.0 if not available.
	 * @see GetHorizontalFieldOfView, GetViewRotation, SetVerticalFieldOfView
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API float GetVerticalFieldOfView() const;

	/**
	 * Get the aspect ratio of the specified video track.
	 *
	 * @param TrackIndex Index of the video track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Aspect ratio.
	 * @see GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackFrameRates, GetVideoTrackType
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API float GetVideoTrackAspectRatio(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the current dimensions of the specified video track.
	 *
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Video dimensions (in pixels).
	 * @see GetVideoTrackAspectRatio, GetVideoTrackFrameRate, GetVideoTrackFrameRates, GetVideoTrackType
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FIntPoint GetVideoTrackDimensions(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the frame rate of the specified video track.
	 *
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Frame rate (in frames per second).
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRates, GetVideoTrackType, SetVideoTrackFrameRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API float GetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the supported range of frame rates of the specified video track.
	 *
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Frame rate range (in frames per second).
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackType
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FFloatRange GetVideoTrackFrameRates(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the type of the specified video track format.
	 *
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @return Video format type string.
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackFrameRates
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FString GetVideoTrackType(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the current view rotation (only for 360 videos).
	 *
	 * @return View rotation, or zero rotator if not available.
	 * @see GetHorizontalFieldOfView, GetVerticalFieldOfView, SetViewRotation
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FRotator GetViewRotation() const;

	/**
	 * Delay of the player's time.
	 *
	 * @return Delay added to the player's time used to manually sync multiple sources.
	 * @see SetTimeDelay
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FTimespan GetTimeDelay() const;

	/**
	 * Check whether the player is in an error state.
	 *
	 * When the player is in an error state, no further operations are possible.
	 * The current media must be closed, and a new media source must be opened
	 * before the player can be used again. Errors are usually caused by faulty
	 * media files or interrupted network connections.
	 *
	 * @see IsReady
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool HasError() const;

	/**
	 * Check whether playback is buffering data.
	 *
	 * @return true if looping, false otherwise.
	 * @see IsConnecting, IsLooping, IsPaused, IsPlaying, IsPreparing, IsReady
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsBuffering() const;

	/**
	 * Check whether the player is currently connecting to a media source.
	 *
	 * @return true if connecting, false otherwise.
	 * @see IsBuffering, IsLooping, IsPaused, IsPlaying, IsPreparing, IsReady
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsConnecting() const;

	/**
	 * Check whether playback is looping.
	 *
	 * @return true if looping, false otherwise.
	 * @see IsBuffering, IsConnecting, IsPaused, IsPlaying, IsPreparing, IsReady, SetLooping
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsLooping() const;

	/**
	 * Check whether playback is currently paused.
	 *
	 * @return true if playback is paused, false otherwise.
	 * @see CanPause, IsBuffering, IsConnecting, IsLooping, IsPaused, IsPlaying, IsPreparing, IsReady, Pause
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsPaused() const;

	/**
	 * Check whether playback has started.
	 *
	 * @return true if playback has started, false otherwise.
	 * @see CanPlay, IsBuffering, IsConnecting, IsLooping, IsPaused, IsPlaying, IsPreparing, IsReady, Play
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsPlaying() const;

	/**
	 * Check whether the media is currently opening or buffering.
	 *
	 * @return true if playback is being prepared, false otherwise.
	 * @see CanPlay, IsBuffering, IsConnecting, IsLooping, IsPaused, IsPlaying, IsReady, Play
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsPreparing() const;

	/**
	 * Whether media is currently closed.
	 *
	 * @return true if media is closed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsClosed() const;

	/**
	 * Check whether media is ready for playback.
	 *
	 * A player is ready for playback if it has a media source opened that
	 * finished preparing and is not in an error state.
	 *
	 * @return true if media is ready, false otherwise.
	 * @see HasError, IsBuffering, IsConnecting, IsLooping, IsPaused, IsPlaying, IsPreparing
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool IsReady() const;

	/**
	 * Open the next item in the current play list.
	 *
	 * The player will start playing the new media source if it was playing
	 * something previously, otherwise it will only open the media source.
	 *
	 * @return true on success, false otherwise.
	 * @see Close, OpenUrl, OpenSource, Play, Previous, SetPlaylist
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Next();

	/**
	 * Opens the specified media file path.
	 *
	 * A return value of true indicates that the player will attempt to open
	 * the media, but it may fail to do so later for other reasons, i.e. if
	 * a connection to the media server timed out. Use the OnMediaOpened and
	 * OnMediaOpenFailed delegates to detect if and when the media is ready!
	 *
	 * @param FilePath The file path to open.
	 * @return true if the file path will be opened, false otherwise.
	 * @see GetUrl, Close, OpenPlaylist, OpenPlaylistIndex, OpenSource, OpenUrl, Reopen
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool OpenFile(const FString& FilePath);

	/**
	 * Open the first media source in the specified play list.
	 *
	 * @param InPlaylist The play list to open.
	 * @return true if the source will be opened, false otherwise.
	 * @see Close, OpenFile, OpenPlaylistIndex, OpenSource, OpenUrl, Reopen
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	bool OpenPlaylist(UMediaPlaylist* InPlaylist)
	{
		return OpenPlaylistIndex(InPlaylist, 0);
	}

	/**
	 * Open a particular media source in the specified play list.
	 *
	 * @param InPlaylist The play list to open.
	 * @param Index The index of the source to open.
	 * @return true if the source will be opened, false otherwise.
	 * @see Close, OpenFile, OpenPlaylist, OpenSource, OpenUrl, Reopen
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool OpenPlaylistIndex(UMediaPlaylist* InPlaylist, int32 Index);

	/**
	 * Open the specified media source.
	 *
	 * A return value of true indicates that the player will attempt to open
	 * the media, but it may fail to do so later for other reasons, i.e. if
	 * a connection to the media server timed out. Use the OnMediaOpened and
	 * OnMediaOpenFailed delegates to detect if and when the media is ready!
	 *
	 * @param MediaSource The media source to open.
	 * @return true if the source will be opened, false otherwise.
	 * @see Close, OpenFile, OpenPlaylist, OpenPlaylistIndex, OpenUrl, Reopen
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool OpenSource(UMediaSource* MediaSource);

	/**
	 * Open the specified media source with supplied options applied.
	 *
	 * A return value of true indicates that the player will attempt to open
	 * the media, but it may fail to do so later for other reasons, i.e. if
	 * a connection to the media server timed out. Use the OnMediaOpened and
	 * OnMediaOpenFailed delegates to detect if and when the media is ready!
	 *
	 * @param MediaSource The media source to open.
	 * @param Options The media player options to apply.
	 * @return true if the source will be opened, false otherwise.
	 * @see Close, OpenFile, OpenPlaylist, OpenPlaylistIndex, OpenUrl, Reopen
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool OpenSourceWithOptions(UMediaSource* MediaSource, const FMediaPlayerOptions& Options);

	/**
	 * Open the specified media source with options using a latent action.
	 *
	 * A result of true indicates that the player successfully completed all requested operations.
	 *
	 * @param MediaSource The media source to open.
	 * @param Options The media player options to apply.
	 * @param bSuccess  All requested operations have completed successfully.
	 * @see Close, OpenFile, OpenPlaylist, OpenPlaylistIndex, OpenUrl, Reopen
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer", meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject"))
	MEDIAASSETS_API void OpenSourceLatent(const UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, UMediaSource* MediaSource, const FMediaPlayerOptions& Options, bool& bSuccess);

	/**
	 * Opens the specified media URL.
	 *
	 * A return value of true indicates that the player will attempt to open
	 * the media, but it may fail to do so later for other reasons, i.e. if
	 * a connection to the media server timed out. Use the OnMediaOpened and
	 * OnMediaOpenFailed delegates to detect if and when the media is ready!
	 *
	 * @param Url The URL to open.
	 * @return true if the URL will be opened, false otherwise.
	 * @see GetUrl, Close, OpenFile, OpenPlaylist, OpenPlaylistIndex, OpenSource, Reopen
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool OpenUrl(const FString& Url);

	/**
	 * Pauses media playback.
	 *
	 * This is the same as setting the playback rate to 0.0.
	 *
	 * @return true if playback is being paused, false otherwise.
	 * @see CanPause, Close, Next, Play, Previous, Rewind, Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Pause();

	/**
	 * Starts media playback.
	 *
	 * This is the same as setting the playback rate to 1.0.
	 *
	 * @return true if playback is starting, false otherwise.
	 * @see CanPlay, GetRate, Next, Pause, Previous, SetRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Play();

	/**
	 * Starts playback from the media opened event, but can be used elsewhere.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API void PlayAndSeek();

	/**
	 * Open the previous item in the current play list.
	 *
	 * The player will start playing the new media source if it was playing
	 * something previously, otherwise it will only open the media source.
	 *
	 * @return true on success, false otherwise.
	 * @see Close, Next, OpenUrl, OpenSource, Play, SetPlaylist
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Previous();

	/**
	 * Reopens the currently opened media or play list.
	 *
	 * @return true if the media will be opened, false otherwise.
	 * @see Close, Open, OpenFile, OpenPlaylist, OpenPlaylistIndex, OpenSource, OpenUrl
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Reopen();

	/**
	 * Rewinds the media to the beginning.
	 *
	 * This is the same as seeking to zero time.
	 *
	 * @return true if rewinding, false otherwise.
	 * @see GetTime, Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Rewind();

	/**
	 * Seeks to the specified playback time.
	 *
	 * @param Time The playback time to set.
	 * @return true on success, false otherwise.
	 * @see GetTime, Rewind
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Seek(const FTimespan& Time);

	/**
	 * Seeks to the specified playback time as soon as possible.
	 *
	 * If the player is currently seeking, the new seek request will be queued
	 * and executed as soon as the current one is completed. It will guarantee that
	 * at least one sample of the previous seek makes it to the sample queue.
	 * if new seek requests are performed in rapid succession (as in scrubbing),
	 * the requests are aggregated and only the most recent request will be performed.
	 * As a result, OnSeekCompleted event is only sent for the request that got performed.
	 *
	 * If the player is not currently seeking, Seek will be called immediately.
	 *
	 * @param Time The playback time to set.
	 * @return true on success, false otherwise.
	 * @see Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool Scrub(const FTimespan& Time);

	/**
	 * Select the active track of the given type.
	 *
	 * The selected track will use its currently active format. Active formats will
	 * be remembered on a per track basis. The first available format is active by
	 * default. To switch the track format, use SetTrackFormat instead.
	 *
	 * @param TrackType The type of track to select.
	 * @param TrackIndex The index of the track to select, or INDEX_NONE to deselect.
	 * @return true if the track was selected, false otherwise.
	 * @see GetNumTracks, GetSelectedTrack, SetTrackFormat
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SelectTrack(EMediaPlayerTrack TrackType, int32 TrackIndex);

	/**
	 * Set the time on which to block.
	 *
	 * If set, this player will block in TickInput or TickFetch until the video sample
	 * for the specified time are actually available.
	 *
	 * @param Time The time to block on, or FTimespan::MinValue to disable.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API void SetBlockOnTime(const FTimespan& Time);

	/**
	 * Set the time range on which to block.
	 *
	 * If set, this player will block in TickInput or TickFetch until the video sample
	 * for the specified time are actually available.
	 *
	 * @param TimeRange The time range to block on, or empty range to disable.
	 */
	MEDIAASSETS_API void SetBlockOnTimeRange(const TRange<FTimespan>& TimeRange);

	/**
	 * Set the name of the desired native player.
	 *
	 * @param PlayerName The name of the player to set.
	 * @see GetDesiredPlayerName
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API void SetDesiredPlayerName(FName PlayerName);

	/**
	 * Enables or disables playback looping.
	 *
	 * @param Looping Whether playback should be looped.
	 * @return true on success, false otherwise.
	 * @see IsLooping
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SetLooping(bool Looping);

	/**
	 * Sets the media options used by the player.
	 *
	 * @param Options Options to pass to the player.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API void SetMediaOptions(const UMediaSource* Options);

	/**
	 * Changes the media's playback rate.
	 *
	 * @param Rate The playback rate to set.
	 * @return true on success, false otherwise.
	 * @see GetRate, SupportsRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SetRate(float Rate);


	/**
	 * Set the volume on the native player if not mixing with Sound Wave asset.
	 *
	 * The SetNativeVolume can be used to change the audio output volume at runtime. Note that
	 * not all media player plug-ins may support native audio output on all platforms.
	 *
	 * @param Volume The volume to set.
	 * @return true on success, false otherwise.
	 * @see NativeAudioOut
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API bool SetNativeVolume(float Volume);

	/**
	 * Set the format on the specified track.
	 *
	 * Selecting the format will not switch to the specified track. To switch
	 * tracks, use SelectTrack instead. If the track is already selected, the
	 * format change will be applied immediately.
	 *
	 * @param TrackType The type of track to update.
	 * @param TrackIndex The index of the track to update.
	 * @param FormatIndex The index of the format to select (must be valid).
	 * @return true if the track was selected, false otherwise.
	 * @see GetNumTrackFormats, GetNumTracks, GetTrackFormat, SelectTrack
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SetTrackFormat(EMediaPlayerTrack TrackType, int32 TrackIndex, int32 FormatIndex);

	/**
	 * Set the frame rate of the specified video track.
	 *
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @param FrameRate The frame rate to set (must be in range of format's supported frame rates).
	 * @return true on success, false otherwise.
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackFrameRates, GetVideoTrackType
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate);

	/**
	 * Set the field of view (only for 360 videos).
	 *
	 * @param Horizontal Horizontal field of view (in Euler degrees).
	 * @param Vertical Vertical field of view (in Euler degrees).
	 * @param Whether the field of view change should be absolute (true) or relative (false).
	 * @return true on success, false otherwise.
	 * @see GetHorizontalFieldOfView, GetVerticalFieldOfView, SetViewRotation
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API bool SetViewField(float Horizontal, float Vertical, bool Absolute);

	/**
	 * Set the view's rotation (only for 360 videos).
	 *
	 * @param Rotation The desired view rotation.
	 * @param Whether the rotation change should be absolute (true) or relative (false).
	 * @return true on success, false otherwise.
	 * @see GetViewRotation, SetViewField
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API bool SetViewRotation(const FRotator& Rotation, bool Absolute);

	/**
	 * Delay of the player's time.
	 *
	 * This setting can be used to manually sync multiple sources.
	 * Set to 1 seconds, if you would like that Player to play 1 second behind its current time.
	 * If the value is too big, it is possible that the player would not hold that frame for that long.
	 * @return true on success, false otherwise.
	 * @see GetTimeDelay
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaPlayer")
	MEDIAASSETS_API void SetTimeDelay(FTimespan TimeDelay);

	/**
	 * Check whether the specified playback rate is supported.
	 *
	 * @param Rate The playback rate to check.
	 * @param Unthinned Whether no frames should be dropped at the given rate.
	 * @see SupportsScrubbing, SupportsSeeking
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SupportsRate(float Rate, bool Unthinned) const;

	/**
	 * Check whether the currently loaded media supports scrubbing.
	 *
	 * @return true if scrubbing is supported, false otherwise.
	 * @see SupportsRate, SupportsSeeking
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SupportsScrubbing() const;

	/**
	 * Check whether the currently loaded media can jump to a certain position.
	 *
	 * @return true if seeking is supported, false otherwise.
	 * @see SupportsRate, SupportsScrubbing
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SupportsSeeking() const;

public:

	/** A delegate that is invoked when playback has reached the end of the media. */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnEndReached;

	/** A delegate that is invoked when a media source has been closed. */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnMediaClosed;

	/**
	 * A delegate that is invoked when a media source has been opened.
	 *
	 * Depending on whether the underlying player implementation opens the media
	 * synchronously or asynchronously, this event may be executed before or
	 * after the call to OpenSource / OpenUrl returns.
	 *
	 * @see OnMediaOpenFailed, OnTracksChanged
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaOpened OnMediaOpened;

	/**
	 * A delegate that is invoked when a media source has failed to open.
	 *
	 * This delegate is only executed if OpenSource / OpenUrl returned true and
	 * the media failed to open asynchronously later. It is not executed if
	 * OpenSource / OpenUrl returned false, indicating an immediate failure.
	 *
	 * @see OnMediaOpened
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaOpenFailed OnMediaOpenFailed;

	/**
	 * A delegate that is invoked when media playback has been resumed.
	 *
	 * @see OnPlaybackSuspended
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnPlaybackResumed;

	/**
	 * A delegate that is invoked when media playback has been suspended.
	 *
	 * @see OnPlaybackResumed
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnPlaybackSuspended;

	/**
	 * A delegate that is invoked when a seek operation completed successfully.
	 *
	 * Depending on whether the underlying player implementation performs seeks
	 * synchronously or asynchronously, this event may be executed before or
	 * after the call to Seek returns.
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnSeekCompleted;

	/**
	 * A delegate that is invoked when the media track collection changed.
	 *
	 * @see OnMediaOpened
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnTracksChanged;

	/**
	 * A delegate that is invoked when the media metadata changed.
	 *
	 * @see OnMediaOpened
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnMetadataChanged;

	/**
	 * A delegate that is invoked when the player starts buffering data.
	 * This event is not sent by all media players.
	 *
	 * @see OnBufferingCompleted
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnBufferingStart;

	/**
	 * A delegate that may be invoked when the player completes buffering data.
	 * This event should not be relied upon since not all media players support this.
	 *
	 * @see OnBufferingStart
	 */
	UPROPERTY(BlueprintAssignable, Category="Media|MediaPlayer", meta = (HideInDetailPanel))
	FOnMediaPlayerMediaEvent OnBufferingCompleted;
public:

	/**
	 * Check whether the player supports playing back of range within the media.
	 *
	 * @return true if playing back a range is supported, false otherwise.
	 * @see GetPlaybackTimeRange, SetPlaybackTimeRange
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SupportsPlaybackTimeRange() const;

	/**
	 * Returns the current playback range of the media.
	 * If playing back a range is not supported, the range returned will be equal
	 * to [ 0, GetDuration() ).
	 * The media may have an implicit default range provided by the container format
	 * or other means without having called SetPlaybackTimeRange().
	 * The media may have internal time values not starting at 0, which are
	 * conveyed by the range.
	 * Since the range may be only a portion of the media, the duration of the
	 * returned range may be less than the media overall duration returned by
	 * GetDuration().
	 * For live video streams the range may change dynamically as new content
	 * becomes available and old content falls off the timeline.
	 *
	 * @param InRangeToGet The type of range to get.
	 *                     `Absolute` returns the media's smallest and largest timeline values.
	 *                       Unless continuously changing in a Live stream this is usually the
	 *                       same as [ 0, GetDuration() ]. The base time does not have to be
	 *                       zero though.
	 *                     `Current` returns the currently set range, which is a subset of the
	 *                       absolute range.
	 * @return The playback range as queried for.
	 * @see SupportsPlaybackTimeRange, SetPlaybackTimeRange, GetDuration
	 */
	MEDIAASSETS_API TRange<FTimespan> GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet);

	/**
	 * Blueprint accessible version of GetPlaybackTimeRange.
	 * This returns the range truncated into a blueprint usable float interval and should not
	 * be used for live streams as 32 bit floats can not store wallclock times with enough precision.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API FFloatInterval GetPlaybackTimeRange(EMediaTimeRangeBPType InRangeToGet);

	/**
	 * Sets a new media playback range.
	 * Has an effect only if SupportsPlaybackTimeRange() returns true and the media supports it.
	 * A live stream cannot be constrained to a range.
	 * The range will be clamped if necessary to be within the media's absolute time range.
	 * Changing the time range may trigger an implicit Seek() depending on where the current
	 * playback position is located with regard to the new range.
	 * Unless prevented by the media a playback range can be cleared by passing an empty range.
	 *
	 * @param InTimeRange The new playback range to set.
	 * @return true if successful, false otherwise.
	 * @see SupportsPlaybackTimeRange, GetPlaybackTimeRange
	 */
	MEDIAASSETS_API bool SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange);

	/**
	 * Blueprint accessible version of SetPlaybackTimeRange().
	 * The range is set through a blueprint usable float interval which may not have enough
	 * precision to represent the range accurately.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlayer")
	MEDIAASSETS_API bool SetPlaybackTimeRange(FFloatInterval InTimeRange);

	/**
	 * Clean up before this object is destroyed. Normally you would not need to call this,
	 * but if you want to clean up resources before garbage collection (e.g. you are in the editor)
	 * then this could be helpful.
	 */
	MEDIAASSETS_API void CleanUpBeforeDestroy();

	/**
	 * Get the Guid associated with this media player
	 *
	 * @return The Guid.
	 */
	const FGuid& GetGuid()
	{
		return PlayerGuid;
	}

	/**
	 * Get information about the media that is playing.
	 * Not all formats support all information.
	 * Some possible common information is in this file,
	 * look for MediaInfoNameSourceNumMips for example.
	 *
	 * @param	InfoName		Name of the information we want.
	 * @returns					Requested information, or empty if not available.
	 */
	MEDIAASSETS_API FVariant GetMediaInfo(FName InfoName) const;

	/**
	 * Templated version of GetMediaInfo.
	 * No need to deal with variants.
	 *
	 * @param	T			Type of the information.
	 * @param	Result		If the information is found and is the correct type, then this will be
	 *						set to its value. It will not be set otherwise.
	 * @param	InfoName	Name of the information.
	 * @returns				True if the information is found, false if not.
	 */
	template<typename T>
	bool GetMediaInfo(T& Result, FName InfoName) const
	{
		FVariant Variant = GetMediaInfo(InfoName);
		if (Variant.GetType() == TVariantTraits<T>::GetType())
		{
			Result = Variant.GetValue<T>();
			return true;
		}

		return false;
	}

	/**
	 * Get meta data contained in the current stream
	 *
	 * @return Map with arrays of IMediaMetaDataItem entries describing any metadata found in the current stream
	 * @note Listen to EMediaEvent::MetadataChanged to catch updates to this data
	 */
	MEDIAASSETS_API TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> GetMediaMetadata() const;

	/**
	 * This is the blueprint accessible version of the GetMediaMetadata.
	 * @return Map with arrays of FMediaMetaDataItem entries describing any metadata found in the current stream
	 * @note Listen to EMediaEvent::MetadataChanged to catch updates to this data
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Metadata")
	MEDIAASSETS_API TMap<FString, FMediaMetadataItemsBPT> GetMediaMetadataItems() const;

	/**
	 * Get the media player facade that manages low-level media players
	 *
	 * @return The media player facade.
	 */
	MEDIAASSETS_API TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe> GetPlayerFacade() const;

	/**
	 * Register player with media module to be ticked
	 *
	 */
	MEDIAASSETS_API void RegisterWithMediaModule();

	/**
	 * When the player goes out of scope, make sure to clean up the clock sink
	 */
	MEDIAASSETS_API void UnregisterWithMediaModule();

	/**
	 * Get the current play list.
	 *
	 * @return The play list.
	 * @see GetPlaylistIndex, GetPlaylist
	 */
	UMediaPlaylist& GetPlaylistRef() const
	{
		EnsurePlaylist();
		return *Playlist;
	}

	/**
	 * Get an event delegate that is invoked when a media event occurred.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UMediaPlayer, FOnMediaEvent, EMediaEvent /*Event*/)
	FOnMediaEvent& OnMediaEvent()
	{
		return MediaEvent;
	}

#if WITH_EDITOR
	/**
	 * Called when PIE has been paused.
	 *
	 * @see ResumePIE
	 */
	MEDIAASSETS_API void PausePIE();

	/**
	 * Called when PIE has been resumed.
	 *
	 * @see PausePIE
	 */
	MEDIAASSETS_API void ResumePIE();
#endif

public:

	//~ UObject interface

	MEDIAASSETS_API virtual void BeginDestroy() override;

	MEDIAASSETS_API virtual FString GetDesc() override;
	MEDIAASSETS_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	MEDIAASSETS_API virtual void PostInitProperties() override;
	MEDIAASSETS_API virtual void PostLoad() override;

#if WITH_EDITOR
	MEDIAASSETS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	/**
	 * These are possible common information for use with GetMediaInfo.
	 * They also describe the type of data that will be needed to retrieve the information.
	 */
	 /** Number of mip map levels in the source. Int32.*/
	static MEDIAASSETS_API FLazyName MediaInfoNameSourceNumMips;
	/** Number of tiles (X and Y) in the source. IntPoint.*/
	static MEDIAASSETS_API FLazyName MediaInfoNameSourceNumTiles;
	/** Start Timecode. SMPTE format (ex: HH:MM:SS:FF) string. */
	static MEDIAASSETS_API FLazyName MediaInfoNameStartTimecodeValue;
	/** Start Timecode frame rate. String. */
	static MEDIAASSETS_API FLazyName MediaInfoNameStartTimecodeFrameRate;
	/** Keyframe interval. Int32. */
	static MEDIAASSETS_API FLazyName MediaInfoNameKeyframeInterval;
	/** Video decoder provider. String. */
	static MEDIAASSETS_API FLazyName MediaInfoNameVideoDecoderProvider;
	/** Audio decoder provider. String. */
	static MEDIAASSETS_API FLazyName MediaInfoNameAudioDecoderProvider;

	/**
	 * Duration of samples to cache ahead of the play head.
	 *
	 * @see CacheBehind, CacheBehindGame
	 */
	UPROPERTY(BlueprintReadWrite, Category=Caching)
	FTimespan CacheAhead;

	/**
	 * Duration of samples to cache behind the play head (when not running as game).
	 *
	 * @see CacheAhead, CacheBehindGame
	 */
	UPROPERTY(BlueprintReadWrite, Category=Caching)
	FTimespan CacheBehind;

	/**
	 * Duration of samples to cache behind the play head (when running as game).
	 *
	 * @see CacheAhead, CacheBehind
	 */
	UPROPERTY(BlueprintReadWrite, Category=Caching)
	FTimespan CacheBehindGame;

public:

	/**
	 * Output any audio via the operating system's sound mixer instead of a Sound Wave asset.
	 *
	 * If enabled, the assigned Sound Wave asset will be ignored. The SetNativeVolume
	 * function can then be used to change the audio output volume at runtime. Note that
	 * not all media player plug-ins may support native audio output on all platforms.
	 *
	 * @see SetNativeVolume
	 */
	UPROPERTY(BlueprintReadWrite, Category=Output, AdvancedDisplay)
	bool NativeAudioOut;

public:

	/**
	 * Automatically start playback after media opened successfully.
	 *
	 * If disabled, listen to the OnMediaOpened Blueprint event to detect when
	 * the media finished opening, and then start playback using the Play function.
	 *
	 * @see OpenFile, OpenPlaylist, OpenPlaylistIndex, OpenSource, OpenUrl, Play
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Playback)
	bool PlayOnOpen;

	/**
	 * Whether playback should shuffle media sources in the play list.
	 *
	 * @see OpenPlaylist, OpenPlaylistIndex
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Playback)
	uint32 Shuffle : 1;

public:
	//
	// IAudioProxyDataFactory interface (for Meta Sound support)
	//

	MEDIAASSETS_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

protected:

	/**
	 * Whether the player should loop when media playback reaches the end.
	 *
	 * Use the SetLooping function to change this value at runtime.
	 *
	 * @see IsLooping, SetLooping
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Playback)
	uint32 Loop:1;

	/**
	 * The play list to use, if any.
	 *
	 * Use the OpenPlaylist or OpenPlaylistIndex function to change this value at runtime.
	 *
	 * @see OpenPlaylist, OpenPlaylistIndex
	 */
	UPROPERTY(BlueprintReadOnly, transient, Category=Playback, BlueprintGetter = GetPlayList)
	mutable TObjectPtr<UMediaPlaylist> Playlist;

	/**
	 * The current index of the source in the play list being played.
	 *
	 * Use the Previous and Next methods to change this value at runtime.
	 *
	 * @see Next, Previous
	 */
	UPROPERTY(BlueprintReadOnly, Category=Playback)
	int32 PlaylistIndex;

	/**
	 * Delay of the player's time.
	 *
	 * @see SetTimeDelay, GetTimeDelay
	 */
	UPROPERTY(BlueprintReadOnly, Category=Playback)
	FTimespan TimeDelay;

protected:

	/**
	 * The initial horizontal field of view (in Euler degrees; default = 90).
	 *
	 * This setting is used only for 360 videos. It determines the portion of the
	 * video that is visible at a time. To modify the field of view at runtime in
	 * Blueprints, use the SetHorizontalFieldOfView function.
	 *
	 * @see GetHorizontalFieldOfView, SetHorizontalFieldOfView, VerticalFieldOfView, ViewRotation
	 */
	UPROPERTY(EditAnywhere, Category=ViewSettings)
	float HorizontalFieldOfView;

	/**
	 * The initial vertical field of view (in Euler degrees; default = 60).
	 *
	 * This setting is used only for 360 videos. It determines the portion of the
	 * video that is visible at a time. To modify the field of view at runtime in
	 * Blueprints, use the SetHorizontalFieldOfView function.
	 *
	 * Please note that some 360 video players may be able to change only the
	 * horizontal field of view, and this setting may be ignored.
	 *
	 * @see GetVerticalFieldOfView, SetVerticalFieldOfView, HorizontalFieldOfView, ViewRotation
	 */
	UPROPERTY(EditAnywhere, Category=ViewSettings)
	float VerticalFieldOfView;

	/**
	 * The initial view rotation.
	 *
	 * This setting is used only for 360 videos. It determines the rotation of
	 * the video's view. To modify the view orientation at runtime in Blueprints,
	 * use the GetViewRotation and SetViewRotation functions.
	 *
	 * Please note that not all players may support video view rotations.
	 *
	 * @see GetViewRotation, SetViewRotation, HorizontalFieldOfView, VerticalFieldOfView
	 */
	UPROPERTY(EditAnywhere, Category=ViewSettings)
	FRotator ViewRotation;

private:
	virtual bool CanBeInCluster() const override { return false; }

	/** Callback for when a media event occurred in the player. */
	MEDIAASSETS_API void HandlePlayerMediaEvent(EMediaEvent Event);

	/** Sets the playlist and properly handles cases when this MediaPlayer object is in disregard for GC set */
	MEDIAASSETS_API void SetPlaylistInternal(UMediaPlaylist* InPlaylist) const;

	/** Open media source with the given options. */
	MEDIAASSETS_API bool OpenSourceInternal(UMediaSource* MediaSource, const FMediaPlayerOptions* Options);

	/** Ensure internal playlist is created */
	MEDIAASSETS_API void EnsurePlaylist() const;

private:

	/** An event delegate that is invoked when a media event occurred. */
	FOnMediaEvent MediaEvent;

	/** The player facade. */
	TSharedPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> PlayerFacade;

	/** The player's globally unique identifier. */
	UPROPERTY()
	FGuid PlayerGuid;

	/** Automatically start playback of next item in play list. */
	bool PlayOnNext;

	/** Has registered with the media module */
	bool RegisteredWithMediaModule;

	/** Proxy instance for integration with Metasound */
	FMediaPlayerProxyPtr Proxy;

#if WITH_EDITORONLY_DATA
public:
	/** Whether this player should stop when entering or exiting PIE. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category=Editor)
	bool AffectedByPIEHandling;

private:
	/** Whether the player was playing in PIE/SIE. */
	bool WasPlayingInPIE;
#endif
};
