// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "MediaPlayerProxyInterface.generated.h"

class UMediaPlayer;
class UMediaSource;
class UMediaTexture;
struct FMediaSourceCacheSettings;

/**
 * The proxy object provides a higher level of insight/control than just the media player.
 * For example, the object owning the player may control the player in some
 * cases, and the proxy allows you and the object to avoid conflicts in control.
 */
UINTERFACE(MinimalAPI)
class UMediaPlayerProxyInterface : public UInterface
{
	GENERATED_BODY()
};

class IMediaPlayerProxyInterface
{
	GENERATED_BODY()

public:
	/**
	 * Get the desired playback rate.
	 * Note that this is not necessarily the actual rate of the player,
	 * merely the desired rate the user wants.
	 */
	virtual float GetProxyRate() const = 0;

	/**
	 * Changes the desired playback rate.
	 *
	 * @param Rate		The playback rate to set.
	 * @return			True on success, false otherwise.
	 */
	virtual bool SetProxyRate(float Rate) = 0;

	/**
	 * Call this to see if you can control the media player, or if the owning object is using it.
	 * 
	 * @return				True if you can control the player.
	 */
	virtual bool IsExternalControlAllowed() = 0;

	/**
	 * Gets the cache settings for the player.
	 */
	virtual const FMediaSourceCacheSettings& GetCacheSettings() const = 0;

	/**
	 * Get the media source for a given index.
	 */
	virtual UMediaSource* ProxyGetMediaSourceFromIndex(int32 Index) const = 0;

	/**
	 * Get a media texture that we can assign a media player to.
	 */
	virtual UMediaTexture* ProxyGetMediaTexture(int32 LayerIndex, int32 TextureIndex) = 0;

	/**
	 * Release a media texture that was retrieved from ProxyGetMediaTexture.
	 */
	virtual void ProxyReleaseMediaTexture(int32 LayerIndex, int32 TextureIndex) = 0;

	/**
	 * Sets the aspect ratio of the proxy based on what the media player is playing.
	 * 
	 * This should be called every frame until it returns true.
	 * It might take a few frames to discover what the aspect ratio is.
	 * Also some proxies might not support aspect ratios,
	 * so they will also return true so you do not call this endlessly.
	 *
	 * @return True if you no longer need to call this.
	 */
	virtual bool ProxySetAspectRatio(UMediaPlayer* InMediaPlayer) = 0;

	/**
	 * Set the blend value for a texture that was retrieved from ProxyGetMediaTexture.
	 */
	virtual void ProxySetTextureBlend(int32 LayerIndex, int32 TextureIndex, float Blend) = 0;
};
