// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MediaPlaylist.generated.h"

class UMediaSource;
struct FFrame;

/**
 * Implements a media play list.
 */
UCLASS(BlueprintType, hidecategories=(Object), MinimalAPI)
class UMediaPlaylist
	: public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Add a media source to the play list.
	 *
	 * @param MediaSource The media source to append.
	 * @return true if the media source was added, false otherwise.
	 * @see AddFile, AddUrl, Insert, RemoveAll, Remove, Replace
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API bool Add(UMediaSource* MediaSource);

	/**
	 * Add a media file path to the play list.
	 *
	 * @param FilePath The file path to add.
	 * @return true if the file was added, false otherwise.
	 * @see Add, AddUrl, Insert, RemoveAll, Remove, Replace
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API bool AddFile(const FString& FilePath);

	/**
	 * Add a media URL to the play list.
	 *
	 * @param Url The URL to add.
	 * @return true if the URL was added, false otherwise.
	 * @see Add, AddFile, Insert, RemoveAll, Remove, Replace
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API bool AddUrl(const FString& Url);

	/**
	 * Get the media source at the specified index.
	 *
	 * @param Index The index of the media source to get.
	 * @return The media source, or nullptr if the index doesn't exist.
	 * @see GetNext, GetRandom
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API UMediaSource* Get(int32 Index);

	/**
	 * Get the next media source in the play list.
	 *
	 * @param InOutIndex Index of the current media source (will contain the new index).
	 * @return The media source after the current one, or nullptr if the list is empty.
	 * @see , GetPrevious, GetRandom
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API UMediaSource* GetNext(int32& InOutIndex);

	/**
	 * Get the previous media source in the play list.
	 *
	 * @param InOutIndex Index of the current media source (will contain the new index).
	 * @return The media source before the current one, or nullptr if the list is empty.
	 * @see , GetNext, GetRandom
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API UMediaSource* GetPrevious(int32& InOutIndex);

	/**
	 * Get a random media source in the play list.
	 *
	 * @param OutIndex Will contain the index of the returned media source.
	 * @return The random media source, or nullptr if the list is empty.
	 * @see Get, GetNext, GetPrevious
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API UMediaSource* GetRandom(int32& OutIndex);

	/**
	 * Insert a media source into the play list at the given position.
	 *
	 * @param MediaSource The media source to insert.
	 * @param Index The index to insert into.
	 * @see Add, Remove, RemoveAll, Replace
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API void Insert(UMediaSource* MediaSource, int32 Index);

	/**
	 * Get the number of media sources in the play list.
	 *
	 * @return Number of media sources.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	int32 Num()
	{
		return Items.Num();
	}

	/**
	 * Remove all occurrences of the given media source in the play list.
	 *
	 * @param MediaSource The media source to remove.
	 * @return true if the media source was removed, false otherwise.
	 * @see Add, Insert, Remove, Replace
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API bool Remove(UMediaSource* MediaSource);

	/**
	 * Remove the media source at the specified position.
	 *
	 * @param Index The index of the media source to remove.
	 * @return true if the media source was removed, false otherwise.
	 * @see Add, Insert, RemoveAll, Replace
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API bool RemoveAt(int32 Index);

	/**
	 * Replace the media source at the specified position.
	 *
	 * @param Index The index of the media source to replace.
	 * @param Replacement The replacement media source.
	 * @return true if the media source was replaced, false otherwise.
	 * @see Add, Insert, RemoveAll, RemoveAt
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaPlaylist")
	MEDIAASSETS_API bool Replace(int32 Index, UMediaSource* Replacement);

protected:

	/** List of media sources to play. */
	UPROPERTY(EditAnywhere, Category=Playlist)
	TArray<TObjectPtr<UMediaSource>> Items;

	virtual bool CanBeInCluster() const override { return false; }
};
