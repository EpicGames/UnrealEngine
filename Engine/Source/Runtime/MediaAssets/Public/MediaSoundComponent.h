// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"

#include "Containers/Array.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "DSP/SpectrumAnalyzer.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/EnvelopeFollower.h"
#include "Sound/SoundClass.h"
#include "Sound/SoundGenerator.h"
#include "MediaAudioResampler.h"
#include "Misc/TransactionallySafeCriticalSection.h"

#include "MediaSoundComponent.generated.h"

class FMediaAudioResampler;
class FMediaPlayerFacade;
class FMediaSoundComponentClockSink;
class IMediaAudioSample;
class IMediaPlayer;
class UMediaPlayer;


/**
 * Available media sound channel types.
 */
UENUM()
enum class EMediaSoundChannels
{
	/** Mono (1 channel). */
	Mono,

	/** Stereo (2 channels). */
	Stereo,

	/** Surround sound (7.1 channels; for UI). */
	Surround
};

UENUM(BlueprintType)
enum class EMediaSoundComponentFFTSize : uint8
{
	Min_64,
	Small_256,
	Medium_512,
	Large_1024,
};

USTRUCT(BlueprintType)
struct FMediaSoundComponentSpectralData
{
	GENERATED_USTRUCT_BODY()

	// The frequency hz of the spectrum value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	float FrequencyHz = 0.0f;

	// The magnitude of the spectrum at this frequency
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpectralData")
	float Magnitude = 0.0f;
};

// Class implements an ISoundGenerator to feed decoded audio to audio renderering async tasks
class FMediaSoundGenerator : public ISoundGenerator
{
public:
	struct FSoundGeneratorParams
	{
		int32 SampleRate = 0;
		int32 NumChannels = 0;

		TSharedPtr<FMediaAudioSampleQueue, ESPMode::ThreadSafe> SampleQueue;
		uint32 PreviousSampleQueueFlushCount = 0;

		bool bSpectralAnalysisEnabled = false;
		bool bEnvelopeFollowingEnabled = false;

		int32 EnvelopeFollowerAttackTime = 0;
		int32 EnvelopeFollowerReleaseTime = 0;

		Audio::FSpectrumAnalyzerSettings SpectrumAnalyzerSettings;
		TArray<float> FrequenciesToAnalyze;

		float CachedRate = 0.0f;
		FTimespan CachedTime;
		FTimespan LastPlaySampleTime;
	};

	FMediaSoundGenerator(FSoundGeneratorParams& InParams);

	virtual ~FMediaSoundGenerator();

	virtual void OnEndGenerate() override;

	virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

	void SetCachedData(float InCachedRate, const FTimespan& InCachedTime);
	void SetLastPlaySampleTime(const FTimespan& InLastPlaySampleTime);

	void SetEnableSpectralAnalysis(bool bInSpectralAnlaysisEnabled);
	void SetEnableEnvelopeFollowing(bool bInEnvelopeFollowingEnabled);

	void SetSpectrumAnalyzerSettings(Audio::FSpectrumAnalyzerSettings::EFFTSize InFFTSize, const TArray<float>& InFrequenciesToAnalyze);
	void SetEnvelopeFollowingSettings(int32 InAttackTimeMsec, int32 InReleaseTimeMsec);

	void SetSampleQueue(TSharedPtr<FMediaAudioSampleQueue, ESPMode::ThreadSafe>& InSampleQueue);

	TArray<FMediaSoundComponentSpectralData> GetSpectralData() const;
	TArray<FMediaSoundComponentSpectralData> GetNormalizedSpectralData() const;
	float GetCurrentEnvelopeValue() const { return CurrentEnvelopeValue; }

	FTimespan GetLastPlayTime() const { return LastPlaySampleTime.Load(); }

private:

	FSoundGeneratorParams Params;

	/** The audio resampler. */
	FMediaAudioResampler Resampler;

	/** Scratch buffer to mix in source audio to from decoder */
	Audio::AlignedFloatBuffer AudioScratchBuffer;

	/** Spectrum analyzer used for analyzing audio in media. */
	mutable Audio::FAsyncSpectrumAnalyzer SpectrumAnalyzer;

	Audio::FEnvelopeFollower EnvelopeFollower;

	TAtomic<float> CachedRate;
	TAtomic<FTimespan> CachedTime;
	TAtomic<FTimespan> LastPlaySampleTime;

	float CurrentEnvelopeValue = 0.0f;
	bool bEnvelopeFollowerSettingsChanged = false;

	mutable FCriticalSection AnalysisCritSect;
	mutable FCriticalSection SampleQueueCritSect;
};

/**
 * Implements a sound component for playing a media player's audio output.
 */
UCLASS(ClassGroup=Media, editinlinenew, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UMediaSoundComponent
	: public USynthComponent
{
	GENERATED_BODY()

public:

	/** Media sound channel type. */
	UPROPERTY(EditAnywhere, Category="Media")
	EMediaSoundChannels Channels;

	/** Dynamically adjust the sample rate if audio and media clock desynchronize. */
	UPROPERTY(EditAnywhere, Category="Media", AdvancedDisplay)
	bool DynamicRateAdjustment;

	/**
	 * Factor for calculating the sample rate adjustment.
	 *
	 * If dynamic rate adjustment is enabled, this number is multiplied with the drift
	 * between the audio and media clock (in 100ns ticks) to determine the adjustment.
	 * that is to be multiplied into the current playrate.
	 */
	UPROPERTY(EditAnywhere, Category="Media", AdvancedDisplay)
	float RateAdjustmentFactor;

	/**
	 * The allowed range of dynamic rate adjustment.
	 *
	 * If dynamic rate adjustment is enabled, and the necessary adjustment
	 * falls outside of this range, audio samples will be dropped.
	 */
	UPROPERTY(EditAnywhere, Category="Media", AdvancedDisplay)
	FFloatRange RateAdjustmentRange;

public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param ObjectInitializer Initialization parameters.
	 */
	MEDIAASSETS_API UMediaSoundComponent(const FObjectInitializer& ObjectInitializer);

	/** Virtual destructor. */
	MEDIAASSETS_API ~UMediaSoundComponent();

public:

	/**
	 * Get the attenuation settings based on the current component settings.
	 *
	 * @param OutAttenuationSettings Will contain the attenuation settings, if available.
	 * @return true if attenuation settings were returned, false if attenuation is disabled.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaSoundComponent", meta=(DisplayName="Get Attenuation Settings To Apply", ScriptName="GetAttenuationSettingsToApply"))
	MEDIAASSETS_API bool BP_GetAttenuationSettingsToApply(FSoundAttenuationSettings& OutAttenuationSettings);

	/**
	 * Get the media player that provides the audio samples.
	 *
	 * @return The component's media player, or nullptr if not set.
	 * @see SetMediaPlayer
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaSoundComponent")
	MEDIAASSETS_API UMediaPlayer* GetMediaPlayer() const;

	virtual USoundClass* GetSoundClass() override
	{
		if (SoundClass)
		{
			return SoundClass;
		}

		if (const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>())
		{
			if (USoundClass* DefaultSoundClass = AudioSettings->GetDefaultMediaSoundClass())
			{
				return DefaultSoundClass;
			}

			if (USoundClass* DefaultSoundClass = AudioSettings->GetDefaultSoundClass())
			{
				return DefaultSoundClass;
			}
		}

		return nullptr;
	}

	/**
	 * Set the media player that provides the audio samples.
	 *
	 * @param NewMediaPlayer The player to set.
	 * @see GetMediaPlayer
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaSoundComponent")
	MEDIAASSETS_API void SetMediaPlayer(UMediaPlayer* NewMediaPlayer);

	/** Turns on spectral analysis of the audio generated in the media sound component. */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaSoundComponent")
	MEDIAASSETS_API void SetEnableSpectralAnalysis(bool bInSpectralAnalysisEnabled);

	/** Sets the settings to use for spectral analysis. */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaSoundComponent")
	MEDIAASSETS_API void SetSpectralAnalysisSettings(TArray<float> InFrequenciesToAnalyze, EMediaSoundComponentFFTSize InFFTSize = EMediaSoundComponentFFTSize::Medium_512);

	/** Retrieves the spectral data if spectral analysis is enabled. */
	UFUNCTION(BlueprintCallable, Category = "TimeSynth")
	MEDIAASSETS_API TArray<FMediaSoundComponentSpectralData> GetSpectralData();

	/** Retrieves and normalizes the spectral data if spectral analysis is enabled. */
	UFUNCTION(BlueprintCallable, Category = "TimeSynth")
	MEDIAASSETS_API TArray<FMediaSoundComponentSpectralData> GetNormalizedSpectralData();

	/** Turns on amplitude envelope following the audio in the media sound component. */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaSoundComponent")
	MEDIAASSETS_API void SetEnableEnvelopeFollowing(bool bInEnvelopeFollowing);

	/** Sets the envelope attack and release times (in ms). */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaSoundComponent")
	MEDIAASSETS_API void SetEnvelopeFollowingsettings(int32 AttackTimeMsec, int32 ReleaseTimeMsec);

	/** Retrieves the current amplitude envelope. */
	UFUNCTION(BlueprintCallable, Category = "TimeSynth")
	MEDIAASSETS_API float GetEnvelopeValue() const;

public:

	/** Adds a clock sink so this can be ticked without the world. */
	MEDIAASSETS_API void AddClockSink();

	/** Removes the clock sink. */
	MEDIAASSETS_API void RemoveClockSink();

	MEDIAASSETS_API void UpdatePlayer();

#if WITH_EDITOR
	/**
	 * Set the component's default media player property.
	 *
	 * @param NewMediaPlayer The player to set.
	 * @see SetMediaPlayer
	 */
	MEDIAASSETS_API void SetDefaultMediaPlayer(UMediaPlayer* NewMediaPlayer);
#endif

public:

	//~ TAttenuatedComponentVisualizer interface

	MEDIAASSETS_API void CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const;

public:

	//~ UActorComponent interface

	MEDIAASSETS_API virtual void OnRegister() override;
	MEDIAASSETS_API virtual void OnUnregister() override;
	MEDIAASSETS_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

public:

	//~ USceneComponent interface

	MEDIAASSETS_API virtual void Activate(bool bReset = false) override;
	MEDIAASSETS_API virtual void Deactivate() override;

public:

	//~ UObject interface
	MEDIAASSETS_API virtual void PostLoad() override;

#if WITH_EDITOR
	MEDIAASSETS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:

	/**
	 * Get the attenuation settings based on the current component settings.
	 *
	 * @return Attenuation settings, or nullptr if attenuation is disabled.
	 */
	MEDIAASSETS_API const FSoundAttenuationSettings* GetSelectedAttenuationSettings() const;

protected:

	//~ USynthComponent interface

	MEDIAASSETS_API virtual bool Init(int32& SampleRate) override;

	MEDIAASSETS_API virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;

protected:

	/**
	 * The media player asset associated with this component.
	 *
	 * This property is meant for design-time convenience. To change the
	 * associated media player at run-time, use the SetMediaPlayer method.
	 *
	 * @see SetMediaPlayer
	 */
	UPROPERTY(EditAnywhere, Category="Media")
	TObjectPtr<UMediaPlayer> MediaPlayer;

private:

	/** The player's current play rate (cached for use on audio thread). */
	float CachedRate;

	/** The player's current time (cached for use on audio thread). */
	FTimespan CachedTime;

	/** Critical section for synchronizing access to PlayerFacadePtr. */
	FTransactionallySafeCriticalSection CriticalSection;

	/** The player that is currently associated with this component. */
	TWeakObjectPtr<UMediaPlayer> CurrentPlayer;

	/** The player facade that's currently providing texture samples. */
	TWeakPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> CurrentPlayerFacade;

	/** Adjusts the output sample rate to synchronize audio and media clock. */
	float RateAdjustment;

	/** Audio sample queue. */
	TSharedPtr<FMediaAudioSampleQueue, ESPMode::ThreadSafe> SampleQueue;

	/* Time of last sample played. */
	FTimespan LastPlaySampleTime;

	/** Which frequencies to analyze. */
	TArray<float> FrequenciesToAnalyze;
	/** Spectrum analyzer used for analyzing audio in media. */
	Audio::FSpectrumAnalyzerSettings SpectrumAnalyzerSettings;
	int32 EnvelopeFollowerAttackTime;
	int32 EnvelopeFollowerReleaseTime;

	/** Whether or not spectral analysis is enabled. */
	bool bSpectralAnalysisEnabled;

	/** Whether or not envelope following is enabled. */
	bool bEnvelopeFollowingEnabled;

	/** Holds our clock sink if available. */
	TSharedPtr<FMediaSoundComponentClockSink, ESPMode::ThreadSafe> ClockSink;

	/** Instance of our media sound generator. This is a non-uobject that is used to feed sink audio to a sound source on the audio render thread (or async task). */
	ISoundGeneratorPtr MediaSoundGenerator;
};
