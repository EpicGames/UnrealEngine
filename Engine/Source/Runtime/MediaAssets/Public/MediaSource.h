// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "IMediaOptions.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Variant.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"
#include "MediaSourceOptions.h"

#include "MediaSource.generated.h"

class FVariant;
class IMediaSourceRendererInterface;
class UMediaSource;
class UTexture;
struct FFrame;

/** Delegate for creating a media source from a string. */
DECLARE_DELEGATE_RetVal_TwoParams(UMediaSource*, FMediaSourceSpawnDelegate, const FString&, UObject*);

/**
 * Abstract base class for media sources.
 *
 * Media sources describe the location and/or settings of media objects that can
 * be played in a media player, such as a video file on disk, a video stream on
 * the internet, or a web cam attached to or built into the target device. The
 * location is encoded as a media URL string, whose URI scheme and optional file
 * extension will be used to locate a suitable media player.
 */
UCLASS(Abstract, editinlinenew, BlueprintType, hidecategories=(Object), MinimalAPI)
class UMediaSource
	: public UObject
	, public IMediaOptions
{
	GENERATED_BODY()

public:

	/**
	 * Get the media source's URL string (must be implemented in child classes).
	 *
	 * @return The media URL.
	 * @see GetProxies
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaSource")
	virtual FString GetUrl() const PURE_VIRTUAL(UMediaSource::GetUrl, return FString(););

	/**
	 * Validate the media source settings (must be implemented in child classes).
	 *
	 * @return true if validation passed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaSource")
	virtual bool Validate() const PURE_VIRTUAL(UMediaSource::Validate, return false;);

#if WITH_EDITOR

	/**
	 * Starts the process to generate a thumbnail.
	 */
	MEDIAASSETS_API void GenerateThumbnail();

	/**
	 * Gets our thumbnail texture, if any.
	 */
	UTexture* GetThumbnail() const { return ThumbnailImage; }

	/**
	 * Sets what the thumbnail texture should be.
	 */
	void SetThumbnail(UTexture* InTexture) { ThumbnailImage = InTexture; }

	/** Gets a string that sums up the media source's description for UI purposes */
	virtual FString GetDescriptionString() const { return GetUrl(); }

	typedef TTuple<FText, FText, FText> FInfoElement;
	
	/**
	 * Gets a list of text containing the media source's configuration and description for UI purposes
	 * @param OutInfoElements A list of elements, where an element is constructed from a group, a label, and a value
	 */
	MEDIAASSETS_API virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const;
#endif // WITH_EDITOR

	/**
	 * Call this to register a callback when someone calls SpawnMediaSourceForString.
	 * This lets you spawn a media source if the file extension matches what you want.
	 * 
	 * @param Extension		File extension to match. This is case insensitive.
	 * @param InDelegate	This will get called if the Url passed into GetMediaSourceForUrl
	 *						matches Extension.
	 */
	static MEDIAASSETS_API void RegisterSpawnFromFileExtension(const FString& Extension, FMediaSourceSpawnDelegate InDelegate);
	
	/**
	 * Call this to unregister a callback set with RegisterSpawnFromFileExtension.
	 *
	 * @param Extension		File extension that the callack was registered with.
	 */
	static MEDIAASSETS_API void UnregisterSpawnFromFileExtension(const FString& Extension);

	/**
	 * Call this to try and create a media source appropriate for the media.
	 *
	 * @param MediaPath		Can be a file location or a Url.
	 * @param Outer			Outer to use for this object.
	 * @return				Media source or nullptr if none are appropriate.
	 */
	static MEDIAASSETS_API UMediaSource* SpawnMediaSourceForString(const FString& MediaPath, UObject* Outer);

public:
	//~ UObject interface
	MEDIAASSETS_API virtual void BeginDestroy() override;

	//~ IMediaOptions interface

	MEDIAASSETS_API virtual FName GetDesiredPlayerName() const override;
	MEDIAASSETS_API virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	MEDIAASSETS_API virtual double GetMediaOption(const FName& Key, double DefaultValue) const override;
	MEDIAASSETS_API virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	MEDIAASSETS_API virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	MEDIAASSETS_API virtual FText GetMediaOption(const FName& Key, const FText& DefaultValue) const override;
	MEDIAASSETS_API virtual TSharedPtr<FDataContainer, ESPMode::ThreadSafe> GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const override;
	MEDIAASSETS_API virtual bool HasMediaOption(const FName& Key) const override;
	MEDIAASSETS_API virtual const UObject* ToUObject() const override;

	/** Set a boolean parameter to pass to the player. */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "SetMediaOption (boolean)"), Category = "Media|MediaSource")
	MEDIAASSETS_API void SetMediaOptionBool(const FName& Key, bool Value);
	/** Set a float parameter to pass to the player. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "SetMediaOption (float)"), Category = "Media|MediaSource")
	MEDIAASSETS_API void SetMediaOptionFloat(const FName& Key, float Value);
	/** Set a double parameter to pass to the player. */
	MEDIAASSETS_API void SetMediaOptionDouble(const FName& Key, double Value);
	/** Set an integer64 parameter to pass to the player. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "SetMediaOption (integer64)"), Category = "Media|MediaSource")
	MEDIAASSETS_API void SetMediaOptionInt64(const FName& Key, int64 Value);
	/** Set a string parameter to pass to the player. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "SetMediaOption (string)"), Category = "Media|MediaSource")
	MEDIAASSETS_API void SetMediaOptionString(const FName& Key, const FString& Value);

private:
	/** Holds our media options. */
	TMap<FName, FVariant> MediaOptionsMap;

	/**
	 * Get the media option specified by the Key as a Variant.
	 * Returns nullptr if the Key does not exist.
	 */
	MEDIAASSETS_API const FVariant* GetMediaOptionDefault(const FName& Key) const;

	/**
	 * Sets the media option specified by Key to the supplied Variant.
	 */
	MEDIAASSETS_API void SetMediaOption(const FName& Key, FVariant& Value);

	/**
	 * Get a mapping of file extensions to spawn delegates.
	 */
	static MEDIAASSETS_API TMap<FString, FMediaSourceSpawnDelegate>& GetSpawnFromFileExtensionDelegates();

#if WITH_EDITORONLY_DATA

	/** The thumbnail image.*/
	UPROPERTY(Transient)
	TObjectPtr<UTexture> ThumbnailImage = nullptr;

	/** Renders thumnbnails for us. */
	UPROPERTY(Transient)
	TObjectPtr<UObject> MediaSourceRenderer = nullptr;

#endif
};
