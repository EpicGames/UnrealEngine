// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MediaSourceOptions.generated.h"

/** Cache settings to pass to the player. */
USTRUCT(BlueprintType)
struct FMediaSourceCacheSettings
{
	GENERATED_USTRUCT_BODY()

	/**
	 * Override the default cache settings.
	 * Currently only the ImgMedia player supports these settings.
	 */
	UPROPERTY(EditAnywhere, Category = "Media Cache")
	bool bOverride = false;

	/**
	 * The cache will fill up with frames that are up to this time from the current time.
	 * E.g. if this is 0.2, and we are at time index 5 seconds,
	 * then we will fill the cache with frames between 5 seconds and 5.2 seconds.
	 */
	UPROPERTY(EditAnywhere, Category = "Media Cache")
	float TimeToLookAhead = 0.2f;


	inline bool operator==(const FMediaSourceCacheSettings& Other) const
	{
		return (Other.bOverride == bOverride) && FMath::IsNearlyEqual(Other.TimeToLookAhead, TimeToLookAhead);
	}

	inline bool operator!=(const FMediaSourceCacheSettings& Other) const
	{
		return !(*this == Other);
	}
};
