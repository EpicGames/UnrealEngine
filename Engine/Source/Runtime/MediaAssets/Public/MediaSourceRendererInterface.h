// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "MediaSourceRendererInterface.generated.h"

class UMediaSource;
class UMediaTexture;

/**
 * Interface for an object that can render a media source to a texture.
 */
UINTERFACE(MinimalAPI)
class UMediaSourceRendererInterface : public UInterface
{
	GENERATED_BODY()
};

class IMediaSourceRendererInterface
{
	GENERATED_BODY()

public:
	
	/**
	 * Open the media source to render a texture for.
	 *
	 * @param	InMediaSource		Media source to play.
	 * @return	Media texture that will hold the image.
	 */
	virtual UMediaTexture* Open(UMediaSource* InMediaSource) = 0;
};
