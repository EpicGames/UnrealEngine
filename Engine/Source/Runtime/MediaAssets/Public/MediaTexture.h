// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/EnumAsByte.h"
#include "Engine/Texture.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "MediaTexture.generated.h"

class FMediaPlayerFacade;
class FMediaTextureClockSink;
class IMediaTextureSample;
class UMediaPlayer;

UENUM()
enum MediaTextureOrientation : int
{
	MTORI_Original					UMETA(DisplayName = "Original (as decoded)"),
	MTORI_CW90						UMETA(DisplayName = "Clockwise 90deg"),
	MTORI_CW180						UMETA(DisplayName = "Clockwise 180deg"),
	MTORI_CW270						UMETA(DisplayName = "Clockwise 270deg"),
};

/**
 * Implements a texture asset for rendering video tracks from UMediaPlayer assets.
 * 
 * note: derives directly from UTexture, not from UTexture2D or UTexture2DDynamic
 *    maybe should have been UTexture2DDynamic?
 */
UCLASS(hidecategories=(Adjustments, Compositing, LevelOfDetail, ImportSettings, Object), MinimalAPI)
class UMediaTexture
	: public UTexture
{
	GENERATED_UCLASS_BODY()

public:

	/** Possible render modes of this media texture. */
	enum class ERenderMode
	{
		Default = 0,
		JustInTime, // Will defer rendering this media texture until its consumer calls its JustInTimeRender function.
	};

public:

	/** The addressing mode to use for the X axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="MediaTexture", meta=(DisplayName="X-axis Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<TextureAddress> AddressX;

	/** The addressing mode to use for the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="MediaTexture", meta=(DisplayName="Y-axis Tiling Method"), AssetRegistrySearchable, AdvancedDisplay)
	TEnumAsByte<TextureAddress> AddressY;

	/** Whether to clear the texture when no media is being played (default = enabled). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="MediaTexture")
	bool AutoClear;

	/** The color used to clear the texture if AutoClear is enabled (default = black). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="MediaTexture")
	FLinearColor ClearColor;

	/** Basic enablement for mip generation (default = false). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MediaTexture", meta = (DisplayName = "Enable Mipmap generation"))
	bool EnableGenMips;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Only 'Enable Mipmap Generation' is needed now. When true, the full mip chain will be generated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Only 'Enable Mipmap Generation' is needed now. When true, the full mip chain will be generated."))
	uint8 NumMips_DEPRECATED;
#endif

	/**
	* Allows the media texture to be used as a regular 2D texture in materials, with Color or Linear Color sampler types as opposed to External.
	* Note however that external textures can remain more efficient on lower-end platforms (OpenGL ES).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MediaTexture", meta = (DisplayName = "Enable New Style Output"))
	bool NewStyleOutput;

	/** Current aspect ratio */
	UPROPERTY(Transient, TextExportTransient, SkipSerialization, BlueprintReadOnly, Category = "MediaTexture", meta = (DisplayName = "Current frame's aspect ratio"))
	float CurrentAspectRatio;

	/** Current media orientation */
	UPROPERTY(Transient, TextExportTransient, SkipSerialization, BlueprintReadOnly, Category = "MediaTexture", meta = (DisplayName = "Current frame's orientation"))
	TEnumAsByte<enum MediaTextureOrientation> CurrentOrientation;

public:

	/**
	 * Gets the current aspect ratio of the texture.
	 *
	 * @return Texture aspect ratio.
	 * @see GetHeight, GetWidth
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaTexture")
	MEDIAASSETS_API float GetAspectRatio() const;

	/**
	 * Gets the current height of the texture.
	 *
	 * @return Texture height (in pixels).
	 * @see GetAspectRatio, GetWidth
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaTexture")
	MEDIAASSETS_API int32 GetHeight() const;

	/**
	 * Gets the current width of the texture.
	 *
	 * @return Texture width (in pixels).
	 * @see GetAspectRatio, GetHeight
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaTexture")
	MEDIAASSETS_API int32 GetWidth() const;

	/**
	 * Gets the current numbe of mips of the texture.
	 *
	 * @return Number of mips.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaTexture")
	MEDIAASSETS_API int32 GetTextureNumMips() const;

	/**
	 * Get the media player that provides the video samples.
	 *
	 * @return The texture's media player, or nullptr if not set.
	 * @see SetMediaPlayer
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaTexture")
	MEDIAASSETS_API UMediaPlayer* GetMediaPlayer() const;

	/**
	 * Set the media player that provides the video samples.
	 *
	 * @param NewMediaPlayer The player to set.
	 * @see GetMediaPlayer
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaTexture")
	MEDIAASSETS_API void SetMediaPlayer(UMediaPlayer* NewMediaPlayer);

	/**
	 * Set the media player that provides the video samples and optionally the sample queue
	 * that receives them. If no sample queue is provided, the media texture creates a new one internally.
	 *
	 * @param NewMediaPlayer The player to set.
	 * @param InSampleQueue Provide an optional sample queue to use instead of creating a new one.
	 * @see GetMediaPlayer
	 */
	MEDIAASSETS_API void SetMediaPlayer(UMediaPlayer* NewMediaPlayer, TSharedPtr<FMediaTextureSampleQueue> InSampleQueue);
	
	/**
	 * Creates a new resource for the texture, and updates any cached references to the resource.
	 * This obviously is just an override to expose to blueprints. Since the base is deprecated in favor
	 * of the new params version, we remove the virtual override and pass the flags we care about here.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaTexture")
	void UpdateResource() { UpdateResourceWithParams(EUpdateResourceFlags::None); }

	/**
	 * Caches the next available sample time from the queue when last rendering was made
	 * @see GetNextSampleTime
	 */
	MEDIAASSETS_API void CacheNextAvailableSampleTime(FTimespan InNextSampleTime);

	/**
	 * Gets the next sample Time. Only valid if GetAvailableSampleCount is greater than 0
	 * @note This value is cached when last render command was executed to keep single consumer requirements.
	 * @return FTimespan of the next sample or FTimespan::MinValue if no sample was available in the queue.
	 * @see GetAvailableSampleCount, CacheNextAvailableSampleTime
	 */
	MEDIAASSETS_API FTimespan GetNextSampleTime() const;

	/**
	 * Gets the number of samples that are available
	 * @return The number of samples in the queue
	 */
	MEDIAASSETS_API int32 GetAvailableSampleCount() const;

#if WITH_EDITOR

	/**
	 * Set the texture's default media player property.
	 *
	 * @param NewMediaPlayer The player to set.
	 * @see SetMediaPlayer
	 */
	MEDIAASSETS_API void SetDefaultMediaPlayer(UMediaPlayer* NewMediaPlayer);

#endif

	/**
	 * Get current aspect ratio of presented frame.
	 * @return Aspect ratio of current frame
	 */
	MEDIAASSETS_API float GetCurrentAspectRatio() const;

	/**
	 * Get current orientation of presented frame.
	 * @return Orientation of current frame
	 */
	MEDIAASSETS_API MediaTextureOrientation GetCurrentOrientation() const;

	/**
	 * Get the texture's mip-map bias, clamped to a legal range.
	 * @return Mip-map bias value
	 */
	MEDIAASSETS_API float GetMipMapBias() const;

	/**
	 * Set texture's mip-map bias, for use by the texture resource sampler.
	 * Note: UpdateResource() should be called afterwards and the material should be notified.
	 *
	 * @param InMipMapBias Bias value.
	 */
	MEDIAASSETS_API void SetMipMapBias(float InMipMapBias);

	/**
	 * Set an override for color space conversions
	 */
	void SetColorSpaceOverride(UE::Color::EColorSpace InColorspaceOverride)
	{
		ColorspaceOverride = InColorspaceOverride;
	}

	/** Renders this media texture. Only has an effect if its RenderMode is ERenderMode::JustInTime */
	MEDIAASSETS_API virtual void JustInTimeRender();

	/** Sets the ERenderMode of this media texture */
	void SetRenderMode(ERenderMode InRenderMode)
	{
		RenderMode = InRenderMode;
	}

	/** Returns the ERenderMode of this media texture */
	ERenderMode GetRenderMode()
	{
		return RenderMode;
	}

public:

	//~ UTexture interface.

	MEDIAASSETS_API virtual void BeginDestroy() override;
	MEDIAASSETS_API virtual FTextureResource* CreateResource() override;
	MEDIAASSETS_API virtual EMaterialValueType GetMaterialType() const override;
	MEDIAASSETS_API virtual float GetSurfaceWidth() const override;
	MEDIAASSETS_API virtual float GetSurfaceHeight() const override;
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	MEDIAASSETS_API virtual FGuid GetExternalTextureGuid() const override;
	MEDIAASSETS_API void SetRenderedExternalTextureGuid(const FGuid& InNewGuid);
	virtual ETextureClass GetTextureClass() const override { return ETextureClass::Other2DNoSource; }
	MEDIAASSETS_API virtual uint32 CalcTextureMemorySizeEnum(ETextureMipCount Enum) const override;
	virtual bool HasRuntimeVariableSRGB() const override { return true; }

public:

	//~ UObject interface.

	MEDIAASSETS_API virtual FString GetDesc() override;
	MEDIAASSETS_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	MEDIAASSETS_API virtual void PostInitProperties() override;
	MEDIAASSETS_API virtual void PostLoad() override;
	MEDIAASSETS_API virtual bool IsPostLoadThreadSafe() const override;

#if WITH_EDITOR
	MEDIAASSETS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:

	/**
	 * Tick the texture resource.
	 *
	 * @param Timecode The current timecode.
	 */
	MEDIAASSETS_API void TickResource(FTimespan Timecode);

	/**
	 * Update the video sample queue, if necessary.
	 * @param InSampleQueue Optional sample queue to use.
	 */
	MEDIAASSETS_API void UpdatePlayerAndQueue(TSharedPtr<FMediaTextureSampleQueue> InSampleQueue = nullptr);

	/** Update sample info */
	MEDIAASSETS_API void UpdateSampleInfo(const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample);

protected:

	/**
	 * The media player asset associated with this texture.
	 *
	 * This property is meant for design-time convenience. To change the
	 * associated media player at run-time, use the SetMediaPlayer method.
	 *
	 * @see SetMediaPlayer
	 */
	UPROPERTY(EditAnywhere, Category="Media")
	TObjectPtr<UMediaPlayer> MediaPlayer;

private:

	friend class FMediaTextureClockSink;

	/** The texture's media clock sink. */
	TSharedPtr<FMediaTextureClockSink, ESPMode::ThreadSafe> ClockSink;

	/** The default external texture GUID if no media player is assigned. */
	FGuid CurrentGuid;

	/** The last Guid that was rendered and registered in the render command*/
	FGuid CurrentRenderedGuid;

	/** The player that is currently associated with this texture. */
	TWeakObjectPtr<UMediaPlayer> CurrentPlayer;

	/** The default external texture GUID if no media player is assigned. */
	const FGuid DefaultGuid;

	/** Current width and height of the resource (in pixels). */
	FIntPoint Dimensions;

	/** Last Tick's width and height of the resource (in pixels). */
	FIntPoint LastDimensions;

	/** The previously clear color. */
	FLinearColor LastClearColor;

	/** The previously used sRGB flag. */
	bool LastSrgb;

	/** True if the texture has been cleared. */
	bool bIsCleared;

	/** Texture sample queue. */
	TSharedPtr<FMediaTextureSampleQueue, ESPMode::ThreadSafe> SampleQueue;

	/** Current size of the resource (in bytes).*/
	SIZE_T Size;

	/** Critical section to protect last rendered guid since it can be read from anywhere. */
	mutable FCriticalSection CriticalSection;

	/** Next available sample time when last render call was made */
	TAtomic<FTimespan> CachedNextSampleTime;

	/** Number of mips in the actual output texture */
	int32 TextureNumMips;

	/** Mip-map bias used by the media texture resource sampler. */
	float MipMapBias;

	/** Current render mode of this media texture. Can be changed using SetRenderMode() */
	ERenderMode RenderMode = ERenderMode::Default;

	/** Colorspace override */
	UE::Color::EColorSpace ColorspaceOverride;
};
