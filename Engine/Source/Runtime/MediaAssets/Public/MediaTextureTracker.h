// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MediaTextureTracker.generated.h"

class AActor;
class IMediaTileVisibilityProvider;
class UMediaTexture;
struct FMediaTextureTrackerObject;

// Workaround to avoid having a comma in the type for the multicast macro.
using FMediaTextureTrackerObjectPtr = TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>;

/** Fires on register/unregister of a tracker object. Game thread only. Texture may be null on unregister if GC'd. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMediaTextureTrackerObjectsChanged, TObjectPtr<UMediaTexture> /*Texture*/, const FMediaTextureTrackerObjectPtr& /*Info*/);

UENUM()
enum class EMediaTextureVisibleMipsTiles : uint8
{
	None = 0,
	Plane = 1,
	Sphere
};

enum class EMediaTextureTargetViewResolution : uint8
{
	None = 0,
	RenderResolution = 1 << 0,
	DisplayResolution = 1 << 1,
	All = RenderResolution | DisplayResolution
};
ENUM_CLASS_FLAGS(EMediaTextureTargetViewResolution);

/** Holds info on a single object. */
struct FMediaTextureTrackerObject
{
	/** Actor that is using our img sequence. */
	TWeakObjectPtr<class AActor> Object;
	/** LOD bias for the mipmap level. */
	float MipMapLODBias;
	/** Specify type of mesh used for visible mips and tiles calculations. */
	EMediaTextureVisibleMipsTiles VisibleMipsTilesCalculations;
	/** Arc size in degrees used for visible tiles calculations, specific to the sphere.*/
	FVector2D MeshRange;
	/* Mip level to upscale into lower quality mips. All levels including and above the specified value will be fully read. */
	int32 MipLevelToUpscale = -1;
	/* When this is enabled Media Plate will automatically try to figure out the best mip to upscale to reduce the load. */
	bool bAdaptivePoleMipUpscaling = false;
	/* Bitwise mask for the target displayed resolution of the media texture (used for mip level estimation). */
	EMediaTextureTargetViewResolution TargetViewResolutionMask = EMediaTextureTargetViewResolution::RenderResolution;

	/** Optional consumer-supplied tile-visibility provider; takes precedence over VisibleMipsTilesCalculations when set. */
	TSharedPtr<IMediaTileVisibilityProvider, ESPMode::ThreadSafe> TileVisibilityProvider;
};

/**
 * Tracks which media textures are used by which objects.
 */
class FMediaTextureTracker
{
public:
	/**
	 * Access engine from here.
	 *
	 * @return Engine.
	 */
	MEDIAASSETS_API static FMediaTextureTracker& Get();

	virtual ~FMediaTextureTracker() = default;

	/**
	 * Each object should call this for each media texture that the object has.
	 *
	 * @param InInfo Object that has the texture.
	 * @param InTexture Texture to register.
	 */
	MEDIAASSETS_API void RegisterTexture(TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>& InInfo, TObjectPtr<UMediaTexture> InTexture);

	/**
	 * Each object should call this for each media texture that the object has.
	 *
	 * @param InInfo Object that has the texture.
	 * @param InTexture Texture to unregister. May be null if the caller has lost the
	 *        pointer (e.g. the texture was garbage-collected before the owner was destroyed);
	 *        in that case the tracker scans for InInfo across all registered textures.
	 */
	MEDIAASSETS_API void UnregisterTexture(TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>& InInfo, TObjectPtr<UMediaTexture> InTexture);

	/**
	 * Get which objects are usnig a specific media texture.
	 *
	 * @param InTexture Media texture to check.
	 * @return Nullptr if nothing found, or a pointer to an array of objects.
	 */
	MEDIAASSETS_API const TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>* GetObjects(TObjectPtr<UMediaTexture> InTexture) const;

	/**
	 * Get a list of media textures we know about.
	 *
	 * @ return List of textures.
	 */
	const TArray<TWeakObjectPtr<UMediaTexture>>& GetTextures()
	{
		check(IsInGameThread());
		return MediaTextures;
	}

	/** Delegate fired after a tracker object has been added for a texture. Subscribe to pick up late registrations. */
	FOnMediaTextureTrackerObjectsChanged& OnObjectRegistered()
	{
		check(IsInGameThread());
		return ObjectRegisteredDelegate;
	}

	/** Delegate fired after a tracker object has been removed for a texture. */
	FOnMediaTextureTrackerObjectsChanged& OnObjectUnregistered()
	{
		check(IsInGameThread());
		return ObjectUnregisteredDelegate;
	}

protected:
	/** Maps a media texture to an array of playback components. */
	TMap<TWeakObjectPtr<UMediaTexture>, TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>> MapTextureToObject;
	/** Array of media textures that we know about. */
	TArray<TWeakObjectPtr<UMediaTexture>> MediaTextures;

	FOnMediaTextureTrackerObjectsChanged ObjectRegisteredDelegate;
	FOnMediaTextureTrackerObjectsChanged ObjectUnregisteredDelegate;
};
