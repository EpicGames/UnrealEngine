// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MediaBlueprintFunctionLibrary.generated.h"

class UObject;
struct FFrame;


/**
 * Filter flags for the EnumerateAudioCaptureDevices BP function.
 */
UENUM(BlueprintType, meta=(BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMediaAudioCaptureDeviceFilter : uint8
{
	None = 0 UMETA(Hidden),

	/** Audio capture cards. */
	Card = 0x1,

	/** Microphone. */
	Microphone = 0x2,

	/** Software device. */
	Software = 0x4,

	/** Unknown audio capture device types. */
	Unknown = 0x8
};

ENUM_CLASS_FLAGS(EMediaAudioCaptureDeviceFilter)


/**
 * Filter flags for the EnumerateVideoCaptureDevices BP function.
 */
UENUM(BlueprintType, meta=(BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMediaVideoCaptureDeviceFilter : uint8
{
	None = 0 UMETA(Hidden),

	/** Video capture card. */
	Card = 0x1,

	/** Software video capture device. */
	Software = 0x2,

	/** Unknown video capture device types. */
	Unknown = 0x4,

	/** Web cam. */
	Webcam = 0x8
};

ENUM_CLASS_FLAGS(EMediaVideoCaptureDeviceFilter)


/**
 * Filter flags for the EnumerateWebcamCaptureDevices BP function.
 */
UENUM(BlueprintType, meta=(BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMediaWebcamCaptureDeviceFilter : uint8
{
	None = 0 UMETA(Hidden),

	/** Depth sensor. */
	DepthSensor = 0x1,

	/** Front facing web cam. */
	Front = 0x2,

	/** Rear facing web cam. */
	Rear = 0x4,

	/** Unknown web cam types. */
	Unknown = 0x8
};

ENUM_CLASS_FLAGS(EMediaWebcamCaptureDeviceFilter)


/**
 * Information about a capture device.
 */
USTRUCT(BlueprintType)
struct FMediaCaptureDevice
{
	GENERATED_USTRUCT_BODY()

	/** Human readable display name. */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Media Capture Device")
	FText DisplayName;

	/** Media URL string for use with media players. */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Media Capture Device")
	FString Url;

	/** Default constructor. */
	FMediaCaptureDevice() { }

	/** Create and initialize a new instance. */
	FMediaCaptureDevice(const FText& InDisplayName, const FString& InUrl)
		: DisplayName(InDisplayName)
		, Url(InUrl)
	{ }
};


/**
 * Blueprint library for Media related functions.
 */
UCLASS(meta=(ScriptName="MediaLibrary"), MinimalAPI)
class UMediaBlueprintFunctionLibrary
	: public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Enumerate available audio capture devices.
	 *
	 * To filter for a specific subset of devices, use the MakeBitmask node
	 * with EMediaAudioCaptureDeviceFilter as the Bitmask Enum.
	 *
	 * @param OutDevices Will contain the available capture devices.
	 * @param Filter The types of capture devices to return (-1 = all).
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Capture")
	static MEDIAASSETS_API void EnumerateAudioCaptureDevices(TArray<FMediaCaptureDevice>& OutDevices, int32 Filter = -1);

	/**
	 * Enumerate available audio capture devices.
	 *
	 * To filter for a specific subset of devices, use the MakeBitmask node
	 * with EMediaVideoCaptureDeviceFilter as the Bitmask Enum.
	 *
	 * @param OutDevices Will contain the available capture devices.
	 * @param Filter The types of capture devices to return (-1 = all).
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Capture")
	static MEDIAASSETS_API void EnumerateVideoCaptureDevices(TArray<FMediaCaptureDevice>& OutDevices, int32 Filter = -1);

	/**
	 * Enumerate available audio capture devices.
	 *
	 * To filter for a specific subset of devices, use the MakeBitmask node
	 * with EMediaWebcamCaptureDeviceFilter as the Bitmask Enum.
	 *
	 * @param OutDevices Will contain the available capture devices.
	 * @param Filter The types of capture devices to return (-1 = all).
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Capture")
	static MEDIAASSETS_API void EnumerateWebcamCaptureDevices(TArray<FMediaCaptureDevice>& OutDevices, int32 Filter = -1);
};
