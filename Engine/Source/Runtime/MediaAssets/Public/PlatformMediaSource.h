// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MediaSource.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "PlatformMediaSource.generated.h"

class FArchive;
class FObjectPreSaveContext;
class UObject;
struct FGuid;


/**
 * A media source that selects other media sources based on target platform.
 *
 * Use this asset to override media sources on a per-platform basis.
 */
UCLASS(BlueprintType, MinimalAPI)
class UPlatformMediaSource
	: public UMediaSource
{
	GENERATED_BODY()

public:

#if WITH_EDITORONLY_DATA

	/** Media sources per platform. */
	UPROPERTY(transient, EditAnywhere, Category=Sources, Meta=(DisplayName="Media Sources"))
	TMap<FString, TObjectPtr<UMediaSource>> PlatformMediaSources;

private:
	/** Blind data encountered at load that could not be mapped to a known platform */
	TMap<FGuid, UMediaSource*> BlindPlatformMediaSources;

#endif

public:
	//~ UObject interface
	MEDIAASSETS_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext);
	MEDIAASSETS_API virtual void Serialize(FArchive& Ar) override;

	//~ UMediaSource interface

	MEDIAASSETS_API virtual FString GetUrl() const override;
	MEDIAASSETS_API virtual bool Validate() const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
public:

	//~ IMediaOptions interface

	MEDIAASSETS_API virtual FName GetDesiredPlayerName() const override;
	MEDIAASSETS_API virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	MEDIAASSETS_API virtual double GetMediaOption(const FName& Key, double DefaultValue) const override;
	MEDIAASSETS_API virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	MEDIAASSETS_API virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	MEDIAASSETS_API virtual FText GetMediaOption(const FName& Key, const FText& DefaultValue) const override;
	MEDIAASSETS_API virtual bool HasMediaOption(const FName& Key) const override;

private:

	/**
	 * Get the media source for the running target platform.
	 *
	 * @return The media source, or nullptr if not set.
	 */
	UMediaSource* GetMediaSource() const;

private:

	/**
	 * Default media source.
	 *
	 * This media source will be used if no source was specified for a target platform.
	 */
	UPROPERTY()
	TObjectPtr<UMediaSource> MediaSource;
};
