// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMediaSource.h"
#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "StreamMediaSource.generated.h"

class UObject;


UCLASS(BlueprintType, MinimalAPI)
class UStreamMediaSource
	: public UBaseMediaSource
{
	GENERATED_BODY()

public:

	/** The URL to the media stream to be played. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Stream, AssetRegistrySearchable)
	FString StreamUrl;

public:

	//~ UMediaSource interface

	MEDIAASSETS_API virtual FString GetUrl() const override;
	MEDIAASSETS_API virtual bool Validate() const override;

#if WITH_EDITOR
	MEDIAASSETS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};
