// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMediaSource.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TimeSynchronizableMediaSource.generated.h"

class UObject;


namespace TimeSynchronizableMedia
{
	/** Name of bUseTimeSynchronization media option. */
	static FName UseTimeSynchronizatioOption("UseTimeSynchronization");
	static FName FrameDelay("FrameDelay");
	static FName TimeDelay("TimeDelay");
	static FName AutoDetect("AutoDetect");
}

/**
 * Base class for media sources that can be synchronized with the engine's timecode.
 */
UCLASS(Abstract, MinimalAPI)
class UTimeSynchronizableMediaSource : public UBaseMediaSource
{
	GENERATED_BODY()
	
public:
	/** Default constructor. */
	MEDIAASSETS_API UTimeSynchronizableMediaSource();

public:

	/**
	 * Synchronize the media with the engine's timecode.
	 * The media player has be able to read timecode.
	 * The media player will try to play the corresponding frame, base on the frame's timecode value.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Synchronization, meta=(DisplayName="Time Synchronization"))
	bool bUseTimeSynchronization;

	/** 
	* When using Time Synchronization, how many frame back should it read. Capped at 4 frames due to the potential issues with buffer sizes.
	* Framelocked modes don't support this feature.
	*/
	UPROPERTY(EditAnywhere, Category=Synchronization, meta=(EditCondition="bUseTimeSynchronization", ClampMin = "0", ClampMax = "4"))
	int32 FrameDelay;

	/** When not using Time Synchronization, how far back it time should it read. */
	UPROPERTY(EditAnywhere, Category=Synchronization, meta=(EditCondition="!bUseTimeSynchronization", ForceUnits=s))
	double TimeDelay;

	/** Whether to autodetect the input or not. */
	UPROPERTY()
	bool bAutoDetectInput = true;

public:
	//~ IMediaOptions interface
	using Super::GetMediaOption;
	MEDIAASSETS_API virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	MEDIAASSETS_API virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	MEDIAASSETS_API virtual double GetMediaOption(const FName& Key, double DefaultValue) const override;
	MEDIAASSETS_API virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	MEDIAASSETS_API virtual bool HasMediaOption(const FName& Key) const override;

	virtual bool SupportsFormatAutoDetection() const { return false; }
};
